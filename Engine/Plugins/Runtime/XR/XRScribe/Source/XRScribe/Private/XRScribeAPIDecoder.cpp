// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRScribeAPIDecoder.h"

namespace UE::XRScribe
{

// helpers

template <typename XrType>
int32 ReadXrTypeList(const FOpenXRAPIPacketBase& Packet, int32 OffsetToListData, uint32 TypeCount, XrType* DstTypeList)
{
	const uint8* const PacketBaseAddress = reinterpret_cast<const uint8*>(&Packet);
	const uint8* const PropertiesBaseAddress = PacketBaseAddress + OffsetToListData;

	const int32 ListBytes = static_cast<int32>(TypeCount * sizeof(XrType));

	FMemory::Memcpy(static_cast<void*>(DstTypeList), PropertiesBaseAddress, ListBytes);

	return ListBytes;
}

FOpenXRCaptureDecoder::FOpenXRCaptureDecoder()
{
	for (ApiDecodeFn& Fn : DecodeFnTable)
	{
		Fn = nullptr;
	}

	DecodeFnTable[(uint32)EOpenXRAPIPacketId::EnumerateApiLayerProperties] = &FOpenXRCaptureDecoder::DecodeEnumerateApiLayerProperties;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::EnumerateInstanceExtensionProperties] = &FOpenXRCaptureDecoder::DecodeEnumerateInstanceExtensionProperties;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::CreateInstance] = &FOpenXRCaptureDecoder::DecodeCreateInstance;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::DestroyInstance] = &FOpenXRCaptureDecoder::DecodeDestroyInstance;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::GetInstanceProperties] = &FOpenXRCaptureDecoder::DecodeGetInstanceProperties;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::PollEvent] = nullptr; // TODO: pending encode support
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::ResultToString] = nullptr; // TODO: pending encode support
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::StructureTypeToString] = nullptr; // TODO: pending encode support
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::GetSystem] = &FOpenXRCaptureDecoder::DecodeGetSystem;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::GetSystemProperties] = &FOpenXRCaptureDecoder::DecodeGetSystemProperties;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::EnumerateEnvironmentBlendModes] = &FOpenXRCaptureDecoder::DecodeEnumerateEnvironmentBlendModes;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::CreateSession] = &FOpenXRCaptureDecoder::DecodeCreateSession;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::DestroySession] = &FOpenXRCaptureDecoder::DecodeDestroySession;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::EnumerateReferenceSpaces] = &FOpenXRCaptureDecoder::DecodeEnumerateReferenceSpaces;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::CreateReferenceSpace] = &FOpenXRCaptureDecoder::DecodeCreateReferenceSpace;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::GetReferenceSpaceBoundsRect] = &FOpenXRCaptureDecoder::DecodeGetReferenceSpaceBoundsRect;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::CreateActionSpace] = &FOpenXRCaptureDecoder::DecodeCreateActionSpace;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::LocateSpace] = &FOpenXRCaptureDecoder::DecodeLocateSpace;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::DestroySpace] = &FOpenXRCaptureDecoder::DecodeDestroySpace;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::EnumerateViewConfigurations] = &FOpenXRCaptureDecoder::DecodeEnumerateViewConfigurations;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::GetViewConfigurationProperties] = &FOpenXRCaptureDecoder::DecodeGetViewConfigurationProperties;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::EnumerateViewConfigurationViews] = &FOpenXRCaptureDecoder::DecodeEnumerateViewConfigurationViews;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::EnumerateSwapchainFormats] = &FOpenXRCaptureDecoder::DecodeEnumerateSwapchainFormats;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::CreateSwapchain] = &FOpenXRCaptureDecoder::DecodeCreateSwapchain;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::DestroySwapchain] = &FOpenXRCaptureDecoder::DecodeDestroySwapchain;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::EnumerateSwapchainImages] = &FOpenXRCaptureDecoder::DecodeEnumerateSwapchainImages;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::AcquireSwapchainImage] = &FOpenXRCaptureDecoder::DecodeAcquireSwapchainImage;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::WaitSwapchainImage] = &FOpenXRCaptureDecoder::DecodeWaitSwapchainImage;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::ReleaseSwapchainImage] = &FOpenXRCaptureDecoder::DecodeReleaseSwapchainImage;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::BeginSession] = &FOpenXRCaptureDecoder::DecodeBeginSession;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::EndSession] = &FOpenXRCaptureDecoder::DecodeEndSession;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::RequestExitSession] = &FOpenXRCaptureDecoder::DecodeRequestExitSession;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::WaitFrame] = &FOpenXRCaptureDecoder::DecodeWaitFrame;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::BeginFrame] = &FOpenXRCaptureDecoder::DecodeBeginFrame;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::EndFrame] = &FOpenXRCaptureDecoder::DecodeEndFrame;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::LocateViews] = &FOpenXRCaptureDecoder::DecodeLocateViews;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::StringToPath] = &FOpenXRCaptureDecoder::DecodeStringToPath;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::PathToString] = &FOpenXRCaptureDecoder::DecodePathToString;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::CreateActionSet] = &FOpenXRCaptureDecoder::DecodeCreateActionSet;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::DestroyActionSet] = &FOpenXRCaptureDecoder::DecodeDestroyActionSet;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::CreateAction] = &FOpenXRCaptureDecoder::DecodeCreateAction;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::DestroyAction] = &FOpenXRCaptureDecoder::DecodeDestroyAction;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::SuggestInteractionProfileBindings] = &FOpenXRCaptureDecoder::DecodeSuggestInteractionProfileBindings;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::AttachSessionActionSets] = &FOpenXRCaptureDecoder::DecodeAttachSessionActionSets;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::GetCurrentInteractionProfile] = &FOpenXRCaptureDecoder::DecodeGetCurrentInteractionProfile;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::GetActionStateBoolean] = &FOpenXRCaptureDecoder::DecodeGetActionStateBoolean;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::GetActionStateFloat] = &FOpenXRCaptureDecoder::DecodeGetActionStateFloat;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::GetActionStateVector2F] = &FOpenXRCaptureDecoder::DecodeGetActionStateVector2f;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::GetActionStatePose] = &FOpenXRCaptureDecoder::DecodeGetActionStatePose;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::SyncActions] = &FOpenXRCaptureDecoder::DecodeSyncActions;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::EnumerateBoundSourcesForAction] = nullptr; // TODO: pending encode support
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::GetInputSourceLocalizedName] = nullptr; // TODO: pending encode support
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::ApplyHapticFeedback] = &FOpenXRCaptureDecoder::DecodeApplyHapticFeedback;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::StopHapticFeedback] = &FOpenXRCaptureDecoder::DecodeStopHapticFeedback;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::InitializeLoaderKHR] = &FOpenXRCaptureDecoder::DecodeInitializeLoaderKHR;
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::GetVisibilityMaskKHR] = &FOpenXRCaptureDecoder::DecodeGetVisibilityMaskKHR;
#if defined(XR_USE_GRAPHICS_API_D3D11)
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::GetD3D11GraphicsRequirementsKHR] = &FOpenXRCaptureDecoder::DecodeGetD3D11GraphicsRequirementsKHR;
#endif
#if defined(XR_USE_GRAPHICS_API_D3D12)
	DecodeFnTable[(uint32)EOpenXRAPIPacketId::GetD3D12GraphicsRequirementsKHR] = &FOpenXRCaptureDecoder::DecodeGetD3D12GraphicsRequirementsKHR;
#endif
}

FOpenXRCaptureDecoder::~FOpenXRCaptureDecoder() {}

bool FOpenXRCaptureDecoder::DecodeDataFromMemory()
{
	const int64 NumBytes = EncodedData.Num();
	int64 CurByteIndex = 0;
	int64 CurPacketIndex = 0;

	while (CurByteIndex < NumBytes)
	{
		const FOpenXRAPIPacketBase* NextPacket = reinterpret_cast<const FOpenXRAPIPacketBase*>(&EncodedData[CurByteIndex]);

		if (NextPacket->Padding0 != FOpenXRAPIPacketBase::MagicPacketByte)
		{
			UE_LOGF(LogXRScribeEmulate, Error, "Encountered invalid magic byte in packet %lld while decoding capture", CurPacketIndex);
			return false;
		}

		if (NextPacket->ApiId < EOpenXRAPIPacketId::EnumerateApiLayerProperties || NextPacket->ApiId > EOpenXRAPIPacketId::NumValidAPIPacketIds)
		{
			UE_LOGF(LogXRScribeEmulate, Error, "Encountered invalid API ID in packet %lld while decoding capture", CurPacketIndex);
			return false;
		}

		check(DecodeFnTable[(uint32)NextPacket->ApiId] != nullptr);

		// We explicitly need `this` to make it clear which instance is being passed to function pointer
		const bool bValid = (this->*DecodeFnTable[(uint32)NextPacket->ApiId])(*NextPacket);
		if (!bValid)
		{
			UE_LOGF(LogXRScribeEmulate, Error, "Encountered invalid XrResult in packet %lld while decoding capture", CurPacketIndex);
			return false;
		}

		CurByteIndex = EncodedData.Tell();
		CurPacketIndex++;
	}

	check(CurByteIndex == NumBytes);

	return true;
}

///////////////
/// packet decoders

bool FOpenXRCaptureDecoder::DecodeEnumerateApiLayerProperties(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::EnumerateApiLayerProperties);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXREnumerateApiLayerPropertiesPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	if (!Data.LayerProperties.IsEmpty())
	{
		ApiLayerProperties = MoveTemp(Data.LayerProperties);
	}

	return true;
}

bool FOpenXRCaptureDecoder::DecodeEnumerateInstanceExtensionProperties(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::EnumerateInstanceExtensionProperties);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXREnumerateInstanceExtensionPropertiesPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	if (Data.LayerName[0] != 0)
	{
		return false;
	}

	if (!Data.ExtensionProperties.IsEmpty())
	{
		InstanceExtensionProperties = MoveTemp(Data.ExtensionProperties);
	}

	return true;
}

bool FOpenXRCaptureDecoder::DecodeCreateInstance(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::CreateInstance);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRCreateInstancePacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	ValidInstanceCreateFlags = Data.CreateFlags;
	RequestedLayerNames = MoveTemp(Data.EnabledLayerNames);
	RequestedExtensionNames = MoveTemp(Data.EnabledExtensionNames);

	return true;
}

bool FOpenXRCaptureDecoder::DecodeDestroyInstance(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::DestroyInstance);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRDestroyInstancePacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing for us to do with this action currently
}

bool FOpenXRCaptureDecoder::DecodeGetInstanceProperties(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::GetInstanceProperties);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRGetInstancePropertiesPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	InstanceProperties = MoveTemp(Data.InstanceProperties);
	return true;
}

bool FOpenXRCaptureDecoder::DecodeGetSystem(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::GetSystem);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRGetSystemPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	SystemGetInfo = MoveTemp(Data.SystemGetInfo);
	return true;
}

bool FOpenXRCaptureDecoder::DecodeGetSystemProperties(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::GetSystemProperties);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRGetSystemPropertiesPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	SystemProperties = MoveTemp(Data.SystemProperties);
	return true;
}

bool FOpenXRCaptureDecoder::DecodeEnumerateEnvironmentBlendModes(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::EnumerateEnvironmentBlendModes);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXREnumerateEnvironmentBlendModesPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	if ((Data.ViewConfigurationType == XR_VIEW_CONFIGURATION_TYPE_PRIMARY_STEREO) && 
		!Data.EnvironmentBlendModes.IsEmpty())
	{
		EnvironmentBlendModes = MoveTemp(Data.EnvironmentBlendModes);
	}
	else
	{
		// TODO: Log error, unsupported view config type
	}

	return true;
}

bool FOpenXRCaptureDecoder::DecodeCreateSession(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::CreateSession);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRCreateSessionPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	SessionCreateInfo = Data.SessionCreateInfo;
	return true;
}

bool FOpenXRCaptureDecoder::DecodeDestroySession(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::DestroySession);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRDestroySessionPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// Nothing to do
}

bool FOpenXRCaptureDecoder::DecodeEnumerateReferenceSpaces(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::EnumerateReferenceSpaces);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXREnumerateReferenceSpacesPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	if (!Data.Spaces.IsEmpty())
	{
		ReferenceSpaceTypes = MoveTemp(Data.Spaces);
	}

	return true;
}

bool FOpenXRCaptureDecoder::DecodeCreateReferenceSpace(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::CreateReferenceSpace);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRCreateReferenceSpacePacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	ReferenceSpaceMap.Add(Data.Space, Data.ReferenceSpaceCreateInfo.referenceSpaceType);
	CreatedReferenceSpaces.Add(Data);
	return true;
}

bool FOpenXRCaptureDecoder::DecodeGetReferenceSpaceBoundsRect(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::GetReferenceSpaceBoundsRect);

	// Failed results are allowed here

	FOpenXRGetReferenceSpaceBoundsRectPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	if (Data.Result == XR_SUCCESS)
	{
		ReferenceSpaceBounds.Add(Data.ReferenceSpaceType, Data.Bounds);
		// TODO: Check for existing bounds associated with reference space?
	}

	return true;
}

bool FOpenXRCaptureDecoder::DecodeCreateActionSpace(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::CreateActionSpace);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRCreateActionSpacePacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	ActionSpaceMap.Add(Data.Space, Data.ActionSpaceCreateInfo.action);
	CreatedActionSpaces.Add(Data);
	return true;
}

bool FOpenXRCaptureDecoder::DecodeLocateSpace(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::LocateSpace);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRLocateSpacePacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	if (!SpaceLocations.Contains(Data.Space))
	{
		SpaceLocations.Add(Data.Space);
	}

	//FLocateSpaceRecord Record{};
	//Record.BaseSpace = Data.BaseSpace;
	//Record.Time = Data.Time;
	//Record.Location = Data.Location;

	SpaceLocations[Data.Space].Add(Data);
	return true;
}

bool FOpenXRCaptureDecoder::DecodeDestroySpace(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::DestroySpace);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRDestroySpacePacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}

bool FOpenXRCaptureDecoder::DecodeEnumerateViewConfigurations(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::EnumerateViewConfigurations);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXREnumerateViewConfigurationsPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	if (!Data.ViewConfigurationTypes.IsEmpty())
	{
		ViewConfigurationTypes = MoveTemp(Data.ViewConfigurationTypes);
	}

	return true;
}

bool FOpenXRCaptureDecoder::DecodeGetViewConfigurationProperties(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::GetViewConfigurationProperties);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRGetViewConfigurationPropertiesPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	ViewConfigurationProperties.Add(Data.ViewConfigurationType, Data.ConfigurationProperties);
	return true;
}

bool FOpenXRCaptureDecoder::DecodeEnumerateViewConfigurationViews(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::EnumerateViewConfigurationViews);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXREnumerateViewConfigurationViewsPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	if (!Data.Views.IsEmpty())
	{
		ViewConfigurationViews.Add(Data.ViewConfigurationType, Data.Views);
	}

	return true;
}

bool FOpenXRCaptureDecoder::DecodeEnumerateSwapchainFormats(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::EnumerateSwapchainFormats);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXREnumerateSwapchainFormatsPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	if (!Data.Formats.IsEmpty())
	{
		SwapchainFormats = MoveTemp(Data.Formats);
	}

	return true;
}

bool FOpenXRCaptureDecoder::DecodeCreateSwapchain(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::CreateSwapchain);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRCreateSwapchainPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// TODO: Actual swapchain creation information not needed...yet
}

bool FOpenXRCaptureDecoder::DecodeDestroySwapchain(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::DestroySwapchain);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRDestroySwapchainPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}

bool FOpenXRCaptureDecoder::DecodeEnumerateSwapchainImages(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::EnumerateSwapchainImages);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXREnumerateSwapchainImagesPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}

bool FOpenXRCaptureDecoder::DecodeAcquireSwapchainImage(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::AcquireSwapchainImage);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRAcquireSwapchainImagePacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}

bool FOpenXRCaptureDecoder::DecodeWaitSwapchainImage(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::WaitSwapchainImage);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRWaitSwapchainImagePacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}
bool FOpenXRCaptureDecoder::DecodeReleaseSwapchainImage(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::ReleaseSwapchainImage);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRReleaseSwapchainImagePacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}
bool FOpenXRCaptureDecoder::DecodeBeginSession(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::BeginSession);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRBeginSessionPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}

bool FOpenXRCaptureDecoder::DecodeEndSession(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::EndSession);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXREndSessionPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}

bool FOpenXRCaptureDecoder::DecodeRequestExitSession(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::RequestExitSession);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRRequestExitSessionPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}

bool FOpenXRCaptureDecoder::DecodeWaitFrame(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::WaitFrame);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRWaitFramePacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	WaitFrames.Add(Data);
	return true;
}

bool FOpenXRCaptureDecoder::DecodeBeginFrame(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::BeginFrame);

	// For xrBeginFrame, XR_FRAME_DISCARDED is also considered a successful result
	if (BasePacket.Result != XR_SUCCESS && BasePacket.Result != XR_FRAME_DISCARDED)
	{
		return false;
	}

	FOpenXRBeginFramePacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}

bool FOpenXRCaptureDecoder::DecodeEndFrame(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::EndFrame);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXREndFramePacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// Not really sure what to do with this wealth of info here!
}

bool FOpenXRCaptureDecoder::DecodeLocateViews(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::LocateViews);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRLocateViewsPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	if (!ViewLocations.Contains(Data.ViewLocateInfo.viewConfigurationType))
	{
		ViewLocations.Add(Data.ViewLocateInfo.viewConfigurationType);
	}

	if (Data.Views.Num() > 0)
	{
		ViewLocations[Data.ViewLocateInfo.viewConfigurationType].Add(Data);
	}

	return true;
}

bool FOpenXRCaptureDecoder::DecodeStringToPath(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::StringToPath);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRStringToPathPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	PathToStringMap.Add(Data.GeneratedPath, FName(ANSI_TO_TCHAR(Data.PathStringToWrite.GetData())));
	return true;

	// TODO: Do we need a bi-directional map at any point?
}

bool FOpenXRCaptureDecoder::DecodePathToString(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::PathToString);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRPathToStringPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}

bool FOpenXRCaptureDecoder::DecodeCreateActionSet(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::CreateActionSet);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRCreateActionSetPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}

bool FOpenXRCaptureDecoder::DecodeDestroyActionSet(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::DestroyActionSet);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRDestroyActionSetPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}

bool FOpenXRCaptureDecoder::DecodeCreateAction(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::CreateAction);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRCreateActionPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	CreatedActions.Add(Data);
	return true;
}

bool FOpenXRCaptureDecoder::DecodeDestroyAction(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::DestroyAction);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRDestroyActionPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}

bool FOpenXRCaptureDecoder::DecodeSuggestInteractionProfileBindings(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::SuggestInteractionProfileBindings);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRSuggestInteractionProfileBindingsPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	check(PathToStringMap.Contains(Data.InteractionProfile));
	StringToSuggestedBindingsMap.Add(PathToStringMap[Data.InteractionProfile], Data.SuggestedBindings);

	return true;
}

bool FOpenXRCaptureDecoder::DecodeAttachSessionActionSets(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::AttachSessionActionSets);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRAttachSessionActionSetsPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}

bool FOpenXRCaptureDecoder::DecodeGetCurrentInteractionProfile(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::GetCurrentInteractionProfile);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRGetCurrentInteractionProfilePacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;

	// nothing to do yet
}

bool FOpenXRCaptureDecoder::DecodeGetActionStateBoolean(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::GetActionStateBoolean);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRGetActionStateBooleanPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	if (!BooleanActionStates.Contains(Data.GetInfoBoolean.action))
	{
		BooleanActionStates.Add(Data.GetInfoBoolean.action);
	}
	BooleanActionStates[Data.GetInfoBoolean.action].Add(Data);

	return true;
}

bool FOpenXRCaptureDecoder::DecodeGetActionStateFloat(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::GetActionStateFloat);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRGetActionStateFloatPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	if (!FloatActionStates.Contains(Data.GetInfoFloat.action))
	{
		FloatActionStates.Add(Data.GetInfoFloat.action);
	}
	FloatActionStates[Data.GetInfoFloat.action].Add(Data);

	return true;
}

bool FOpenXRCaptureDecoder::DecodeGetActionStateVector2f(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::GetActionStateVector2F);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRGetActionStateVector2fPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	if (!VectorActionStates.Contains(Data.GetInfoVector2f.action))
	{
		VectorActionStates.Add(Data.GetInfoVector2f.action);
	}
	VectorActionStates[Data.GetInfoVector2f.action].Add(Data);

	return true;
}

bool FOpenXRCaptureDecoder::DecodeGetActionStatePose(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::GetActionStatePose);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRGetActionStatePosePacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	if (!PoseActionStates.Contains(Data.GetInfoPose.action))
	{
		PoseActionStates.Add(Data.GetInfoPose.action);
	}
	PoseActionStates[Data.GetInfoPose.action].Add(Data);

	return true;
}

bool FOpenXRCaptureDecoder::DecodeSyncActions(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::SyncActions);

	// For xrSyncActions, XR_SESSION_NOT_FOCUSED is also considered a successful result
	if (BasePacket.Result != XR_SUCCESS && BasePacket.Result != XR_SESSION_NOT_FOCUSED)
	{
		return false;
	}

	FOpenXRSyncActionsPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;

	SyncActions.Add(Data);
	return true;
}

//bool FOpenXRCaptureDecoder::DecodeEnumerateBoundSourcesForAction(const FOpenXRAPIPacketBase& BasePacket)
//{
//
//}
//bool FOpenXRCaptureDecoder::DecodeGetInputSourceLocalizedName(const FOpenXRAPIPacketBase& BasePacket)
//{
//
//}

bool FOpenXRCaptureDecoder::DecodeApplyHapticFeedback(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::ApplyHapticFeedback);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRApplyHapticFeedbackPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;
}

bool FOpenXRCaptureDecoder::DecodeStopHapticFeedback(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::StopHapticFeedback);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRStopHapticFeedbackPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;
}

bool FOpenXRCaptureDecoder::DecodeInitializeLoaderKHR(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::InitializeLoaderKHR);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRInitializeLoaderKHRPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;
}

bool FOpenXRCaptureDecoder::DecodeGetVisibilityMaskKHR(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::GetVisibilityMaskKHR);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRGetVisibilityMaskKHRPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;
}

#if defined(XR_USE_GRAPHICS_API_D3D11)
bool FOpenXRCaptureDecoder::DecodeGetD3D11GraphicsRequirementsKHR(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::GetD3D11GraphicsRequirementsKHR);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRGetD3D11GraphicsRequirementsKHRPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;
}
#endif

#if defined(XR_USE_GRAPHICS_API_D3D12)
bool FOpenXRCaptureDecoder::DecodeGetD3D12GraphicsRequirementsKHR(const FOpenXRAPIPacketBase& BasePacket)
{
	check(BasePacket.ApiId == EOpenXRAPIPacketId::GetD3D12GraphicsRequirementsKHR);

	if (BasePacket.Result != XR_SUCCESS)
	{
		return false;
	}

	FOpenXRGetD3D12GraphicsRequirementsKHRPacket Data(XrResult::XR_ERROR_RUNTIME_FAILURE);

	EncodedData << Data;
	return true;
}
#endif

}
