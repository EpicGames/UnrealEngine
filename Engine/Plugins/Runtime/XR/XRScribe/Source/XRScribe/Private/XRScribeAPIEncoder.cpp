// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRScribeAPIEncoder.h"

#include "CoreGlobals.h"
#include "HAL/PlatformTime.h"
#include "OpenXRCore.h"
#include "XRScribeFileFormat.h"

namespace UE::XRScribe
{

FOpenXRCaptureEncoder::FOpenXRCaptureEncoder()
{
}

FOpenXRCaptureEncoder::~FOpenXRCaptureEncoder()
{
}

template <typename PacketType>
void FOpenXRCaptureEncoder::WritePacketData(PacketType& Data)
{
	// TODO Support piping data to a thread-safe queue in order to avoid locking

	FWriteScopeLock Lock(CaptureWriteMutex);
	*this << Data;
}

uint32 NumElementsToWrite(uint32 CapacityCount, const uint32* GeneratedCount)
{
	check(GeneratedCount != nullptr);
	return FMath::Min(CapacityCount, *GeneratedCount);
}

void FOpenXRCaptureEncoder::EncodeEnumerateApiLayerProperties(const XrResult Result, const uint32_t propertyCapacityInput,
	const uint32_t* propertyCountOutput, const XrApiLayerProperties* properties)
{
	FOpenXREnumerateApiLayerPropertiesPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::EnumerateApiLayerProperties);

	Data.LayerProperties.Append(properties, NumElementsToWrite(propertyCapacityInput, propertyCountOutput));

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeEnumerateInstanceExtensionProperties(const XrResult Result, const char* layerName,
	const uint32_t propertyCapacityInput, const uint32_t* propertyCountOutput, const XrExtensionProperties* properties)
{
	FOpenXREnumerateInstanceExtensionPropertiesPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::EnumerateInstanceExtensionProperties);

	if (layerName != nullptr)
	{
		FCStringAnsi::Strncpy(Data.LayerName.GetData(), layerName, XR_MAX_API_LAYER_NAME_SIZE);
	}
	else
	{
		Data.LayerName[0] = 0;
	}

	Data.ExtensionProperties.Append(properties, NumElementsToWrite(propertyCapacityInput, propertyCountOutput));

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeCreateInstance(const XrResult Result, const XrInstanceCreateInfo* createInfo, 
	const XrInstance* instance)
{
	FOpenXRCreateInstancePacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::CreateInstance);

	check(createInfo != nullptr);
	check(instance != nullptr);

	Data.CreateFlags = createInfo->createFlags;
	Data.ApplicationInfo = createInfo->applicationInfo;
	
	Data.EnabledLayerNames.AddDefaulted(createInfo->enabledApiLayerCount);
	for (uint32 ApiLayerIndex = 0; ApiLayerIndex < createInfo->enabledApiLayerCount; ApiLayerIndex++)
	{
		FCStringAnsi::Strncpy(
			Data.EnabledLayerNames[ApiLayerIndex].GetData(), 
			createInfo->enabledApiLayerNames[ApiLayerIndex], 
			XR_MAX_API_LAYER_NAME_SIZE);
	}

	Data.EnabledExtensionNames.AddDefaulted(createInfo->enabledExtensionCount);
	for (uint32 ExtensionIndex = 0; ExtensionIndex < createInfo->enabledExtensionCount; ExtensionIndex++)
	{
		FCStringAnsi::Strncpy(
			Data.EnabledExtensionNames[ExtensionIndex].GetData(),
			createInfo->enabledExtensionNames[ExtensionIndex],
			XR_MAX_EXTENSION_NAME_SIZE);
	}

	Data.GeneratedInstance = *instance;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeDestroyInstance(const XrResult Result, const XrInstance instance)
{
	FOpenXRDestroyInstancePacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::DestroyInstance);

	Data.Instance = instance;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeGetInstanceProperties(const XrResult Result, const XrInstance instance,
	const XrInstanceProperties* instanceProperties)
{
	FOpenXRGetInstancePropertiesPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::GetInstanceProperties);

	Data.Instance = instance;
	Data.InstanceProperties = *instanceProperties;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodePollEvent(const XrResult Result, const XrInstance instance, const XrEventDataBuffer* eventData)
{
	// We don't actually need to capture the events in order to emulate the device.
	// The Session lifecycle is well-defined.
	// TODO: in case we have a capture inspector tool, we do need to capture the call data
}

void FOpenXRCaptureEncoder::EncodeResultToString(const XrResult Result, const XrInstance instance, const XrResult value, const char buffer[XR_MAX_RESULT_STRING_SIZE])
{
	// We can own printing of results in our emulation layer
	// TODO: in case we have a capture inspector tool, we do need to capture the call data
}

void FOpenXRCaptureEncoder::EncodeStructureTypeToString(const XrResult Result, const XrInstance instance, const XrStructureType value, const char buffer[XR_MAX_STRUCTURE_NAME_SIZE])
{
	// We can own printing of structure types in our emulation layer
	// TODO: in case we have a capture inspector tool, we do need to capture the call data. But we don't use this in UE.
}

void FOpenXRCaptureEncoder::EncodeGetSystem(const XrResult Result, const XrInstance instance, const XrSystemGetInfo* getInfo, const XrSystemId* systemId)
{
	FOpenXRGetSystemPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::GetSystem);

	Data.Instance = instance;
	Data.SystemGetInfo = *getInfo;
	Data.SystemId = *systemId;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeGetSystemProperties(const XrResult Result, const XrInstance instance, const XrSystemId systemId, const XrSystemProperties* properties)
{
	FOpenXRGetSystemPropertiesPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::GetSystemProperties);

	Data.Instance = instance;
	Data.SystemId = systemId;
	Data.SystemProperties = *properties;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeEnumerateEnvironmentBlendModes(const XrResult Result, const XrInstance instance, const XrSystemId systemId, XrViewConfigurationType viewConfigurationType, const uint32_t environmentBlendModeCapacityInput, const uint32_t* environmentBlendModeCountOutput, const XrEnvironmentBlendMode* environmentBlendModes)
{
	FOpenXREnumerateEnvironmentBlendModesPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::EnumerateEnvironmentBlendModes);

	Data.Instance = instance;
	Data.SystemId = systemId;
	Data.ViewConfigurationType = viewConfigurationType;

	Data.EnvironmentBlendModes.Append(environmentBlendModes, NumElementsToWrite(environmentBlendModeCapacityInput, environmentBlendModeCountOutput));

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeCreateSession(const XrResult Result, const XrInstance instance, const XrSessionCreateInfo* createInfo, const XrSession* session)
{
	FOpenXRCreateSessionPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::CreateSession);

	Data.Instance = instance;
	Data.SessionCreateInfo = *createInfo;
	Data.Session = *session;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeDestroySession(const XrResult Result, const XrSession session)
{
	FOpenXRDestroySessionPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::DestroySession);

	Data.Session = session;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeEnumerateReferenceSpaces(const XrResult Result, const XrSession session, const uint32_t spaceCapacityInput, const uint32_t* spaceCountOutput, const XrReferenceSpaceType* spaces)
{
	FOpenXREnumerateReferenceSpacesPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::EnumerateReferenceSpaces);

	Data.Session = session;

	Data.Spaces.Append(spaces, NumElementsToWrite(spaceCapacityInput, spaceCountOutput));

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeCreateReferenceSpace(const XrResult Result, const XrSession session, const XrReferenceSpaceCreateInfo* createInfo, const XrSpace* space)
{
	FOpenXRCreateReferenceSpacePacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::CreateReferenceSpace);

	Data.Session = session;
	Data.ReferenceSpaceCreateInfo = *createInfo;
	Data.Space = *space;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeGetReferenceSpaceBoundsRect(const XrResult Result, const XrSession session, const XrReferenceSpaceType referenceSpaceType, const XrExtent2Df* bounds)
{
	FOpenXRGetReferenceSpaceBoundsRectPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::GetReferenceSpaceBoundsRect);

	Data.Session = session;
	Data.ReferenceSpaceType = referenceSpaceType;
	Data.Bounds = *bounds;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeCreateActionSpace(const XrResult Result, const XrSession session, const XrActionSpaceCreateInfo* createInfo, const XrSpace* space)
{
	FOpenXRCreateActionSpacePacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::CreateActionSpace);

	Data.Session = session;
	Data.ActionSpaceCreateInfo = *createInfo;
	Data.Space = *space;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeLocateSpace(const XrResult Result, const XrSpace space, const XrSpace baseSpace, const XrTime time, const XrSpaceLocation* location)
{
	FOpenXRLocateSpacePacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::LocateSpace);

	Data.Space = space;
	Data.BaseSpace = baseSpace;
	Data.Time = time;
	Data.Location = *location;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeDestroySpace(const XrResult Result, const XrSpace space)
{
	FOpenXRDestroySpacePacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::DestroySpace);

	Data.Space = space;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeEnumerateViewConfigurations(const XrResult Result, const XrInstance instance, const XrSystemId systemId, const uint32_t viewConfigurationTypeCapacityInput, const uint32_t* viewConfigurationTypeCountOutput, const XrViewConfigurationType* viewConfigurationTypes)
{
	FOpenXREnumerateViewConfigurationsPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::EnumerateViewConfigurations);

	Data.Instance = instance;
	Data.SystemId = systemId;
	Data.ViewConfigurationTypes.Append(viewConfigurationTypes, NumElementsToWrite(viewConfigurationTypeCapacityInput, viewConfigurationTypeCountOutput));

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeGetViewConfigurationProperties(const XrResult Result, const XrInstance instance, const XrSystemId systemId, const XrViewConfigurationType viewConfigurationType, const XrViewConfigurationProperties* configurationProperties)
{
	FOpenXRGetViewConfigurationPropertiesPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::GetViewConfigurationProperties);

	Data.Instance = instance;
	Data.SystemId = systemId;
	Data.ViewConfigurationType = viewConfigurationType;
	Data.ConfigurationProperties = *configurationProperties;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeEnumerateViewConfigurationViews(const XrResult Result, const XrInstance instance, const XrSystemId systemId, const XrViewConfigurationType viewConfigurationType, const uint32_t viewCapacityInput, const uint32_t* viewCountOutput, const XrViewConfigurationView* views)
{
	FOpenXREnumerateViewConfigurationViewsPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::EnumerateViewConfigurationViews);

	Data.Instance = instance;
	Data.SystemId = systemId;
	Data.ViewConfigurationType = viewConfigurationType;
	Data.Views.Append(views, NumElementsToWrite(viewCapacityInput, viewCountOutput));

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeEnumerateSwapchainFormats(const XrResult Result, const XrSession session, const uint32_t formatCapacityInput, const uint32_t* formatCountOutput, int64_t* formats)
{
	FOpenXREnumerateSwapchainFormatsPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::EnumerateSwapchainFormats);

	Data.Session = session;
	Data.Formats.Append(formats, NumElementsToWrite(formatCapacityInput, formatCountOutput));

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeCreateSwapchain(const XrResult Result, const XrSession session, const XrSwapchainCreateInfo* createInfo, const XrSwapchain* swapchain)
{
	FOpenXRCreateSwapchainPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::CreateSwapchain);

	Data.Session = session;
	Data.SwapchainCreateInfo = *createInfo;
	Data.Swapchain = *swapchain;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeDestroySwapchain(const XrResult Result, const XrSwapchain swapchain)
{
	FOpenXRDestroySwapchainPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::DestroySwapchain);

	Data.Swapchain = swapchain;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeEnumerateSwapchainImages(const XrResult Result, const XrSwapchain swapchain, const uint32_t imageCapacityInput, const uint32_t* imageCountOutput, const XrSwapchainImageBaseHeader* images)
{
	FOpenXREnumerateSwapchainImagesPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::EnumerateSwapchainImages);

	Data.Swapchain = swapchain;
	Data.Images.Append(images, NumElementsToWrite(imageCapacityInput, imageCountOutput));

	// TODO: We actually need to check the real type here, and enumerate that
	// I guess the 'real' image might not be super important for emulation

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeAcquireSwapchainImage(const XrResult Result, const XrSwapchain swapchain, const XrSwapchainImageAcquireInfo* acquireInfo, const uint32_t* index)
{
	FOpenXRAcquireSwapchainImagePacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::AcquireSwapchainImage);

	Data.Swapchain = swapchain;
	Data.AcquireInfo = *acquireInfo;
	Data.Index = *index;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeWaitSwapchainImage(const XrResult Result, const XrSwapchain swapchain, const XrSwapchainImageWaitInfo* waitInfo)
{
	FOpenXRWaitSwapchainImagePacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::WaitSwapchainImage);

	Data.Swapchain = swapchain;
	Data.WaitInfo = *waitInfo;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeReleaseSwapchainImage(const XrResult Result, const XrSwapchain swapchain, const XrSwapchainImageReleaseInfo* releaseInfo)
{
	FOpenXRReleaseSwapchainImagePacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::ReleaseSwapchainImage);

	Data.Swapchain = swapchain;
	Data.ReleaseInfo = *releaseInfo;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeBeginSession(const XrResult Result, const XrSession session, const XrSessionBeginInfo* beginInfo)
{
	FOpenXRBeginSessionPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::BeginSession);

	Data.Session = session;
	Data.SessionBeginInfo = *beginInfo;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeEndSession(const XrResult Result, const XrSession session)
{
	FOpenXREndSessionPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::EndSession);

	Data.Session = session;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeRequestExitSession(const XrResult Result, const XrSession session)
{
	FOpenXRRequestExitSessionPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::RequestExitSession);

	Data.Session = session;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeWaitFrame(const XrResult Result, const XrSession session, const XrFrameWaitInfo* frameWaitInfo, const XrFrameState* frameState)
{
	FOpenXRWaitFramePacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::WaitFrame);

	Data.Session = session;
	Data.FrameWaitInfo = *frameWaitInfo;
	Data.FrameState = *frameState;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeBeginFrame(const XrResult Result, const XrSession session, const XrFrameBeginInfo* frameBeginInfo)
{
	FOpenXRBeginFramePacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::BeginFrame);

	Data.Session = session;
	Data.FrameBeginInfo = *frameBeginInfo;

	WritePacketData(Data);
}
void FOpenXRCaptureEncoder::EncodeEndFrame(const XrResult Result, const XrSession session, const XrFrameEndInfo* frameEndInfo)
{
	FOpenXREndFramePacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::EndFrame);

	Data.Session = session;
	Data.DisplayTime = frameEndInfo->displayTime;
	Data.EnvironmentBlendMode = frameEndInfo->environmentBlendMode;

	TArray<XrCompositionLayerQuad> QuadLayers;
	TArray<XrCompositionLayerProjection> ProjectionLayers;
	TArray<XrCompositionLayerProjectionView> ProjectionViews;

	Data.Layers.Reserve(frameEndInfo->layerCount);
	for (uint32 LayerIndex = 0; LayerIndex < frameEndInfo->layerCount; LayerIndex++)
	{
		Data.Layers.Add(*frameEndInfo->layers[LayerIndex]);

		if ((*frameEndInfo->layers[LayerIndex]).type == XR_TYPE_COMPOSITION_LAYER_QUAD)
		{
			const XrCompositionLayerQuad* QuadLayer = reinterpret_cast<const XrCompositionLayerQuad*>(frameEndInfo->layers[LayerIndex]);
			QuadLayers.Add(*QuadLayer);
		}
		else if ((*frameEndInfo->layers[LayerIndex]).type == XR_TYPE_COMPOSITION_LAYER_PROJECTION)
		{
			const XrCompositionLayerProjection* ProjectionLayer = reinterpret_cast<const XrCompositionLayerProjection*>(frameEndInfo->layers[LayerIndex]);
			ProjectionLayers.Add(*ProjectionLayer);

			for (uint32 ViewIndex = 0; ViewIndex < ProjectionLayer->viewCount; ViewIndex++)
			{
				ProjectionViews.Add(ProjectionLayer->views[ViewIndex]);
			}
		}
		else
		{
			// TODO: more informative failure
			check(0);
		}
	}
	Data.QuadLayers = MoveTemp(QuadLayers);
	Data.ProjectionLayers = MoveTemp(ProjectionLayers);
	Data.ProjectionViews = MoveTemp(ProjectionViews);

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeLocateViews(const XrResult Result, const XrSession session, const XrViewLocateInfo* viewLocateInfo, const XrViewState* viewState, const uint32_t viewCapacityInput, const uint32_t* viewCountOutput, const XrView* views)
{
	FOpenXRLocateViewsPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::LocateViews);

	Data.Session = session;
	Data.ViewLocateInfo = *viewLocateInfo;
	Data.ViewState = *viewState;
	Data.Views.Append(views, NumElementsToWrite(viewCapacityInput, viewCountOutput));

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeStringToPath(const XrResult Result, const XrInstance instance, const char* pathString, const XrPath* path)
{
	FOpenXRStringToPathPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::StringToPath);

	Data.Instance = instance;
	FCStringAnsi::Strncpy(Data.PathStringToWrite.GetData(), pathString, XR_MAX_PATH_LENGTH);
	Data.GeneratedPath = *path;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodePathToString(const XrResult Result, const XrInstance instance, const XrPath path, const uint32_t bufferCapacityInput, const uint32_t* bufferCountOutput, const char* buffer)
{
	FOpenXRPathToStringPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::PathToString);

	Data.Instance = instance;
	Data.ExistingPath = path;

	const uint32 MaxBufferLen = FMath::Min((uint32)XR_MAX_PATH_LENGTH, NumElementsToWrite(bufferCapacityInput, bufferCountOutput));
	if (MaxBufferLen > 0)
	{
		FCStringAnsi::Strncpy(Data.PathStringToRead.GetData(), buffer, MaxBufferLen);
	}

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeCreateActionSet(const XrResult Result, const XrInstance instance, const XrActionSetCreateInfo* createInfo, const XrActionSet* actionSet)
{
	FOpenXRCreateActionSetPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::CreateActionSet);

	Data.Instance = instance;
	Data.ActionSetCreateInfo = *createInfo;
	Data.ActionSet = *actionSet;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeDestroyActionSet(const XrResult Result, const XrActionSet actionSet)
{
	FOpenXRDestroyActionSetPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::DestroyActionSet);

	Data.ActionSet = actionSet;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeCreateAction(const XrResult Result, const XrActionSet actionSet, const XrActionCreateInfo* createInfo, const XrAction* action)
{
	FOpenXRCreateActionPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::CreateAction);
	
	Data.ActionSet = actionSet;
	FCStringAnsi::Strncpy(Data.ActionName.GetData(), createInfo->actionName, XR_MAX_ACTION_NAME_SIZE);
	Data.ActionType = createInfo->actionType;
	if (createInfo->countSubactionPaths > 0)
	{
		Data.SubactionPaths.Append(createInfo->subactionPaths, createInfo->countSubactionPaths);
	}
	FCStringAnsi::Strncpy(Data.LocalizedActionName.GetData(), createInfo->localizedActionName, XR_MAX_LOCALIZED_ACTION_NAME_SIZE);
	Data.Action = *action;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeDestroyAction(const XrResult Result, const XrAction action)
{
	FOpenXRDestroyActionPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::DestroyAction);

	Data.Action = action;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeSuggestInteractionProfileBindings(const XrResult Result, const XrInstance instance, const XrInteractionProfileSuggestedBinding* suggestedBindings)
{
	FOpenXRSuggestInteractionProfileBindingsPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::SuggestInteractionProfileBindings);

	Data.Instance = instance;
	Data.InteractionProfile = suggestedBindings->interactionProfile;
	if (suggestedBindings->countSuggestedBindings > 0)
	{
		Data.SuggestedBindings.Append(suggestedBindings->suggestedBindings, suggestedBindings->countSuggestedBindings);
	}

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeAttachSessionActionSets(const XrResult Result, const XrSession session, const XrSessionActionSetsAttachInfo* attachInfo)
{
	FOpenXRAttachSessionActionSetsPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::AttachSessionActionSets);

	Data.Session = session;
	if (attachInfo->countActionSets > 0)
	{
		Data.ActionSets.Append(attachInfo->actionSets, attachInfo->countActionSets);
	}

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeGetCurrentInteractionProfile(const XrResult Result, const XrSession session, const XrPath topLevelUserPath, const XrInteractionProfileState* interactionProfile)
{
	FOpenXRGetCurrentInteractionProfilePacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::GetCurrentInteractionProfile);

	Data.Session = session;
	Data.TopLevelUserPath = topLevelUserPath;
	Data.InteractionProfile = *interactionProfile;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeGetActionStateBoolean(const XrResult Result, const XrSession session, const XrActionStateGetInfo* getInfo, const XrActionStateBoolean* state)
{
	FOpenXRGetActionStateBooleanPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::GetActionStateBoolean);

	Data.Session = session;
	Data.GetInfoBoolean = *getInfo;
	Data.BooleanState = *state;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeGetActionStateFloat(const XrResult Result, const XrSession session, const XrActionStateGetInfo* getInfo, const XrActionStateFloat* state)
{
	FOpenXRGetActionStateFloatPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::GetActionStateFloat);

	Data.Session = session;
	Data.GetInfoFloat = *getInfo;
	Data.FloatState = *state;

	WritePacketData(Data);
}
void FOpenXRCaptureEncoder::EncodeGetActionStateVector2f(const XrResult Result, const XrSession session, const XrActionStateGetInfo* getInfo, const XrActionStateVector2f* state)
{
	FOpenXRGetActionStateVector2fPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::GetActionStateVector2F);

	Data.Session = session;
	Data.GetInfoVector2f = *getInfo;
	Data.Vector2fState = *state;

	WritePacketData(Data);
}
void FOpenXRCaptureEncoder::EncodeGetActionStatePose(const XrResult Result, const XrSession session, const XrActionStateGetInfo* getInfo, const XrActionStatePose* state)
{
	FOpenXRGetActionStatePosePacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::GetActionStatePose);

	Data.Session = session;
	Data.GetInfoPose = *getInfo;
	Data.PoseState = *state;

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeSyncActions(const XrResult Result, const XrSession session, const	XrActionsSyncInfo* syncInfo)
{
	FOpenXRSyncActionsPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::SyncActions);

	Data.Session = session;
	if (syncInfo->countActiveActionSets > 0)
	{
		Data.ActiveActionSets.Append(syncInfo->activeActionSets, syncInfo->countActiveActionSets);
	}

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeEnumerateBoundSourcesForAction(const XrResult Result, const XrSession session, const XrBoundSourcesForActionEnumerateInfo* enumerateInfo, const uint32_t sourceCapacityInput, const uint32_t* sourceCountOutput, const XrPath* sources)
{
	// TODO: We don't actually need to encode this because we don't use this yet in UE

	//FOpenXREnumerateBoundSourcesForActionPacket Data(Result);
	//check(Data.ApiId == EOpenXRAPIPacketId::EnumerateBoundSourcesForAction);

	//Data.Session = session;
	//Data.EnumerateInfo = *enumerateInfo;
	//if (sourceCountOutput != nullptr && *sourceCountOutput > 0)
	//{
	//	Data.Sources.Append(sources, *sourceCountOutput);
	//}

	//WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeGetInputSourceLocalizedName(const XrResult Result, const XrSession session, const XrInputSourceLocalizedNameGetInfo* getInfo, const uint32_t bufferCapacityInput, const uint32_t* bufferCountOutput, const char* buffer)
{
	// TODO: We don't actually need to encode this because we don't use this yet in UE

	//FOpenXRGetInputSourceLocalizedNamePacket Data(Result);
	//check(Data.ApiId == EOpenXRAPIPacketId::GetInputSourceLocalizedName);

	//Data.Session = session;
	//Data.NameGetInfo = *getInfo;
	//if (bufferCountOutput != nullptr && *bufferCountOutput > 0)
	//{
	//	const uint32 MaxBufferLen = FMath::Min((uint32)XR_MAX_PATH_LENGTH, *bufferCountOutput);
	//	FCStringAnsi::Strncpy(Data.LocalizedName.GetData(), buffer, MaxBufferLen);
	//}

	//WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeApplyHapticFeedback(const XrResult Result, const XrSession session, const XrHapticActionInfo* hapticActionInfo, const XrHapticBaseHeader* hapticFeedback)
{
	FOpenXRApplyHapticFeedbackPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::ApplyHapticFeedback);

	Data.Session = session;
	Data.HapticActionInfo = *hapticActionInfo;
	Data.HapticFeedback = *hapticFeedback;

	// TODO: We might want to record all the feedback in the future, but its probably good enough to acknowledge the API being called

	WritePacketData(Data);
}

void FOpenXRCaptureEncoder::EncodeStopHapticFeedback(const XrResult Result, const XrSession session, const XrHapticActionInfo* hapticActionInfo)
{
	FOpenXRStopHapticFeedbackPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::StopHapticFeedback);

	Data.Session = session;
	Data.HapticActionInfo = *hapticActionInfo;

	WritePacketData(Data);
}

// XR_KHR_loader_init
void FOpenXRCaptureEncoder::EncodeInitializeLoaderKHR(const XrResult Result, const XrLoaderInitInfoBaseHeaderKHR* loaderInitInfo)
{
	FOpenXRInitializeLoaderKHRPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::InitializeLoaderKHR);

	Data.LoaderInitInfo = *loaderInitInfo;

	check(0);

	WritePacketData(Data);
}

// XR_KHR_visibility_mask
void FOpenXRCaptureEncoder::EncodeGetVisibilityMaskKHR(const XrResult Result, const XrSession session, const XrViewConfigurationType viewConfigurationType, const uint32_t viewIndex, const XrVisibilityMaskTypeKHR visibilityMaskType, const XrVisibilityMaskKHR* visibilityMask)
{
	FOpenXRGetVisibilityMaskKHRPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::GetVisibilityMaskKHR);

	Data.Session = session;
	Data.ViewConfigurationType = viewConfigurationType;
	Data.ViewIndex = viewIndex;
	Data.VisibilityMaskType = visibilityMaskType;

	Data.Vertices.Append(visibilityMask->vertices, NumElementsToWrite(visibilityMask->vertexCapacityInput, &visibilityMask->vertexCountOutput));
	Data.Indices.Append(visibilityMask->indices, NumElementsToWrite(visibilityMask->indexCapacityInput, &visibilityMask->indexCountOutput));

	WritePacketData(Data);
}

#if defined(XR_USE_GRAPHICS_API_D3D11)
void FOpenXRCaptureEncoder::EncodeGetD3D11GraphicsRequirementsKHR(const XrResult Result, const XrInstance instance, const XrSystemId systemId, const XrGraphicsRequirementsD3D11KHR* graphicsRequirements)
{
	FOpenXRGetD3D11GraphicsRequirementsKHRPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::GetD3D11GraphicsRequirementsKHR);

	Data.Instance = instance;
	Data.SystemId = systemId;
	Data.GraphicsRequirementsD3D11 = *graphicsRequirements;

	WritePacketData(Data);
}
#endif

#if defined(XR_USE_GRAPHICS_API_D3D12)
void FOpenXRCaptureEncoder::EncodeGetD3D12GraphicsRequirementsKHR(const XrResult Result, const XrInstance instance, const XrSystemId systemId, const XrGraphicsRequirementsD3D12KHR* graphicsRequirements)
{
	FOpenXRGetD3D12GraphicsRequirementsKHRPacket Data(Result);
	check(Data.ApiId == EOpenXRAPIPacketId::GetD3D12GraphicsRequirementsKHR);

	Data.Instance = instance;
	Data.SystemId = systemId;
	Data.GraphicsRequirementsD3D12 = *graphicsRequirements;

	WritePacketData(Data);
}
#endif

} // namespace UE::XRScribe