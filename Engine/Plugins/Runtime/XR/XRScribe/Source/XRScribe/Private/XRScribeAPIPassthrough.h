// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenXRPlatformRHI.h"

namespace UE::XRScribe
{

struct FOpenXRAPIPassthrough
{
	// Global
	PFN_xrGetInstanceProcAddr GetInstanceProcAddr = nullptr;
	PFN_xrEnumerateApiLayerProperties EnumerateApiLayerProperties = nullptr;
	PFN_xrEnumerateInstanceExtensionProperties EnumerateInstanceExtensionProperties = nullptr;
	PFN_xrCreateInstance CreateInstance = nullptr;

	// Instance
	PFN_xrDestroyInstance DestroyInstance = nullptr;
	PFN_xrGetInstanceProperties GetInstanceProperties = nullptr;
	PFN_xrPollEvent PollEvent = nullptr;
	PFN_xrResultToString ResultToString = nullptr;
	PFN_xrStructureTypeToString StructureTypeToString = nullptr;
	PFN_xrGetSystem GetSystem = nullptr;
	PFN_xrGetSystemProperties GetSystemProperties = nullptr;
	PFN_xrEnumerateEnvironmentBlendModes EnumerateEnvironmentBlendModes = nullptr;
	PFN_xrCreateSession CreateSession = nullptr;
	PFN_xrDestroySession DestroySession = nullptr;
	PFN_xrEnumerateReferenceSpaces EnumerateReferenceSpaces = nullptr;
	PFN_xrCreateReferenceSpace CreateReferenceSpace = nullptr;
	PFN_xrGetReferenceSpaceBoundsRect GetReferenceSpaceBoundsRect = nullptr;
	PFN_xrCreateActionSpace CreateActionSpace = nullptr;
	PFN_xrLocateSpace LocateSpace = nullptr;
	PFN_xrDestroySpace DestroySpace = nullptr;
	PFN_xrEnumerateViewConfigurations EnumerateViewConfigurations = nullptr;
	PFN_xrGetViewConfigurationProperties GetViewConfigurationProperties = nullptr;
	PFN_xrEnumerateViewConfigurationViews EnumerateViewConfigurationViews = nullptr;
	PFN_xrEnumerateSwapchainFormats EnumerateSwapchainFormats = nullptr;
	PFN_xrCreateSwapchain CreateSwapchain = nullptr;
	PFN_xrDestroySwapchain DestroySwapchain = nullptr;
	PFN_xrEnumerateSwapchainImages EnumerateSwapchainImages = nullptr;
	PFN_xrAcquireSwapchainImage AcquireSwapchainImage = nullptr;
	PFN_xrWaitSwapchainImage WaitSwapchainImage = nullptr;
	PFN_xrReleaseSwapchainImage ReleaseSwapchainImage = nullptr;
	PFN_xrBeginSession BeginSession = nullptr;
	PFN_xrEndSession EndSession = nullptr;
	PFN_xrRequestExitSession RequestExitSession = nullptr;
	PFN_xrWaitFrame WaitFrame = nullptr;
	PFN_xrBeginFrame BeginFrame = nullptr;
	PFN_xrEndFrame EndFrame = nullptr;
	PFN_xrLocateViews LocateViews = nullptr;
	PFN_xrStringToPath StringToPath = nullptr;
	PFN_xrPathToString PathToString = nullptr;
	PFN_xrCreateActionSet CreateActionSet = nullptr;
	PFN_xrDestroyActionSet DestroyActionSet = nullptr;
	PFN_xrCreateAction CreateAction = nullptr;
	PFN_xrDestroyAction DestroyAction = nullptr;
	PFN_xrSuggestInteractionProfileBindings SuggestInteractionProfileBindings = nullptr;
	PFN_xrAttachSessionActionSets AttachSessionActionSets = nullptr;
	PFN_xrGetCurrentInteractionProfile GetCurrentInteractionProfile = nullptr;
	PFN_xrGetActionStateBoolean GetActionStateBoolean = nullptr;
	PFN_xrGetActionStateFloat GetActionStateFloat = nullptr;
	PFN_xrGetActionStateVector2f GetActionStateVector2f = nullptr;
	PFN_xrGetActionStatePose GetActionStatePose = nullptr;
	PFN_xrSyncActions SyncActions = nullptr;
	PFN_xrEnumerateBoundSourcesForAction EnumerateBoundSourcesForAction = nullptr;
	PFN_xrGetInputSourceLocalizedName GetInputSourceLocalizedName = nullptr;
	PFN_xrApplyHapticFeedback ApplyHapticFeedback = nullptr;
	PFN_xrStopHapticFeedback StopHapticFeedback = nullptr;

	// Global extensions
	// XR_KHR_loader_init
	PFN_xrInitializeLoaderKHR InitializeLoaderKHR = nullptr;

	// Instance extensions

	// XR_KHR_visibility_mask
	PFN_xrGetVisibilityMaskKHR GetVisibilityMaskKHR = nullptr;
#if defined(XR_USE_GRAPHICS_API_D3D11)
// XR_KHR_D3D11_enable
	PFN_xrGetD3D11GraphicsRequirementsKHR GetD3D11GraphicsRequirementsKHR = nullptr;
#endif
#if defined(XR_USE_GRAPHICS_API_D3D12)
	// XR_KHR_D3D12_enable
	PFN_xrGetD3D12GraphicsRequirementsKHR GetD3D12GraphicsRequirementsKHR = nullptr;
#endif

};

} // namespace UE::XRScribe
