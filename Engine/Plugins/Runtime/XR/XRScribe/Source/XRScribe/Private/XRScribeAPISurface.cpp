// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRScribeAPISurface.h"

#include "Containers/Map.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/IPluginManager.h"
#include "Logging/LogMacros.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#include "Misc/Paths.h"
#include "OpenXRCore.h"
#include "Templates/SharedPointer.h"
#include "XRScribeCaptureLayer.h"
#include "XRScribeEmulationLayer.h"
#include "XRScribeDeveloperSettings.h"

DEFINE_LOG_CATEGORY_STATIC(LogXRScribeAPI, Log, All);

namespace UE::XRScribe
{

// In order to use this, it has to be set from an INI. We need to
// know the value super early, in order to re-direct the initial OpenXR API calls
// Example addition to DefaultEngine.ini
//
//	[/Script/XRScribe.XRScribeDeveloperSettings]
//	XRScribe.RunMode = 1
//
// Can be set manually, or via the plugin's project settings page. Defaults to 1 (emulate).

enum class EXRScribeRunMode : int32
{
	/** Capture API calls */
	Capture,
	/** Use captured API state to emulate device */
	Emulate,
};

static EXRScribeRunMode XRScribeRunMode = EXRScribeRunMode::Emulate;
static FAutoConsoleVariableRef CVarXRScribeRunMode(TEXT("XRScribe.RunMode"),
	reinterpret_cast<int32&>(XRScribeRunMode),
	TEXT("Toggle the mode XRScribe operates in (capture or emulate)"),
	ECVF_ReadOnly);

static TMap<FString, PFN_xrVoidFunction> GlobalFunctionMap;
static TMap<FString, PFN_xrVoidFunction> InstanceFunctionMap;
static TMap<FString, FString> ExtensionFunctionGateMap;

XRAPI_ATTR XrResult XRAPI_CALL xrGetInstanceProcAddr(
	XrInstance                                  instance,
	const char* name,
	PFN_xrVoidFunction* function)
{
	// We're not obliged to do anything wrt varifying validity of the instance handle
	// but it could be useful for debugging in the future

	if (instance == XR_NULL_HANDLE)
	{
		PFN_xrVoidFunction* itr = GlobalFunctionMap.Find(name);
		if (itr)
		{
			*function = *itr;
			return XrResult::XR_SUCCESS;
		}
		else
		{
			*function = nullptr;
			return XrResult::XR_ERROR_HANDLE_INVALID;
		}

	}
	else
	{
		PFN_xrVoidFunction* itr = InstanceFunctionMap.Find(name);
		if (itr)
		{
			if (ExtensionFunctionGateMap.Contains(name))
			{
				IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
				if (!APISurface.GetActiveLayer()->SupportsInstanceExtension(StringCast<ANSICHAR>(*ExtensionFunctionGateMap[name]).Get()))
				{
					*function = nullptr;
					return XrResult::XR_ERROR_FUNCTION_UNSUPPORTED;
				}
			}

			*function = *itr;
			return XrResult::XR_SUCCESS;
		}
		else
		{
			*function = nullptr;
			return XrResult::XR_ERROR_FUNCTION_UNSUPPORTED;
		}
	}
}

// Global Functions

XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateApiLayerProperties(
	uint32_t                                    propertyCapacityInput,
	uint32_t*									propertyCountOutput,
	XrApiLayerProperties*						properties)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerEnumerateApiLayerProperties(propertyCapacityInput, propertyCountOutput, properties);
}

XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateInstanceExtensionProperties(
	const char*									layerName,
	uint32_t                                    propertyCapacityInput,
	uint32_t*									propertyCountOutput,
	XrExtensionProperties*						properties)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerEnumerateInstanceExtensionProperties(layerName, propertyCapacityInput, propertyCountOutput, properties);
}

XRAPI_ATTR XrResult XRAPI_CALL xrCreateInstance(
	const XrInstanceCreateInfo*					createInfo,
	XrInstance*									instance)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerCreateInstance(createInfo, instance);
}

// Instance functions

XRAPI_ATTR XrResult XRAPI_CALL xrDestroyInstance(
	XrInstance                                  instance)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerDestroyInstance(instance);
}

XRAPI_ATTR XrResult XRAPI_CALL xrGetInstanceProperties(
	XrInstance                                  instance,
	XrInstanceProperties* instanceProperties)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerGetInstanceProperties(instance, instanceProperties);
}

XRAPI_ATTR XrResult XRAPI_CALL xrPollEvent(
	XrInstance                                  instance,
	XrEventDataBuffer*							eventData)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerPollEvent(instance, eventData);
}

XRAPI_ATTR XrResult XRAPI_CALL xrResultToString(
	XrInstance                                  instance,
	XrResult                                    value,
	char                                        buffer[XR_MAX_RESULT_STRING_SIZE])
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerResultToString(instance, value, buffer);
}

XRAPI_ATTR XrResult XRAPI_CALL xrStructureTypeToString(
	XrInstance                                  instance,
	XrStructureType                             value,
	char                                        buffer[XR_MAX_STRUCTURE_NAME_SIZE])
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerStructureTypeToString(instance, value, buffer);
}

XRAPI_ATTR XrResult XRAPI_CALL xrGetSystem(
	XrInstance                                  instance,
	const XrSystemGetInfo*						getInfo,
	XrSystemId*									systemId)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerGetSystem(instance, getInfo, systemId);
}

XRAPI_ATTR XrResult XRAPI_CALL xrGetSystemProperties(
	XrInstance                                  instance,
	XrSystemId                                  systemId,
	XrSystemProperties*							properties)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerGetSystemProperties(instance, systemId, properties);
}

XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateEnvironmentBlendModes(
	XrInstance                                  instance,
	XrSystemId                                  systemId,
	XrViewConfigurationType                     viewConfigurationType,
	uint32_t                                    environmentBlendModeCapacityInput,
	uint32_t*									environmentBlendModeCountOutput,
	XrEnvironmentBlendMode*						environmentBlendModes)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerEnumerateEnvironmentBlendModes(instance, systemId, viewConfigurationType, environmentBlendModeCapacityInput, environmentBlendModeCountOutput,environmentBlendModes);
}

XRAPI_ATTR XrResult XRAPI_CALL xrCreateSession(
	XrInstance                                  instance,
	const XrSessionCreateInfo*					createInfo,
	XrSession*									session)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerCreateSession(instance, createInfo, session);
}

XRAPI_ATTR XrResult XRAPI_CALL xrDestroySession(
	XrSession                                   session)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerDestroySession(session);
}

XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateReferenceSpaces(
	XrSession                                   session,
	uint32_t                                    spaceCapacityInput,
	uint32_t*									spaceCountOutput,
	XrReferenceSpaceType*						spaces)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerEnumerateReferenceSpaces(session, spaceCapacityInput, spaceCountOutput, spaces);
}

XRAPI_ATTR XrResult XRAPI_CALL xrCreateReferenceSpace(
	XrSession                                   session,
	const XrReferenceSpaceCreateInfo*			createInfo,
	XrSpace*									space)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerCreateReferenceSpace(session, createInfo, space);
}

XRAPI_ATTR XrResult XRAPI_CALL xrGetReferenceSpaceBoundsRect(
	XrSession                                   session,
	XrReferenceSpaceType                        referenceSpaceType,
	XrExtent2Df*								bounds)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerGetReferenceSpaceBoundsRect(session, referenceSpaceType, bounds);
}

XRAPI_ATTR XrResult XRAPI_CALL xrCreateActionSpace(
	XrSession                                   session,
	const XrActionSpaceCreateInfo*				createInfo,
	XrSpace*									space)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerCreateActionSpace(session, createInfo, space);
}

XRAPI_ATTR XrResult XRAPI_CALL xrLocateSpace(
	XrSpace                                     space,
	XrSpace                                     baseSpace,
	XrTime                                      time,
	XrSpaceLocation*							location)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerLocateSpace(space, baseSpace, time, location);
}

XRAPI_ATTR XrResult XRAPI_CALL xrDestroySpace(
	XrSpace                                     space)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerDestroySpace(space);
}

XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateViewConfigurations(
	XrInstance                                  instance,
	XrSystemId                                  systemId,
	uint32_t                                    viewConfigurationTypeCapacityInput,
	uint32_t*									viewConfigurationTypeCountOutput,
	XrViewConfigurationType*					viewConfigurationTypes)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerEnumerateViewConfigurations(instance, systemId, viewConfigurationTypeCapacityInput, viewConfigurationTypeCountOutput, viewConfigurationTypes);
}

XRAPI_ATTR XrResult XRAPI_CALL xrGetViewConfigurationProperties(
	XrInstance                                  instance,
	XrSystemId                                  systemId,
	XrViewConfigurationType                     viewConfigurationType,
	XrViewConfigurationProperties*				configurationProperties)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerGetViewConfigurationProperties(instance, systemId, viewConfigurationType, configurationProperties);
}

XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateViewConfigurationViews(
	XrInstance                                  instance,
	XrSystemId                                  systemId,
	XrViewConfigurationType                     viewConfigurationType,
	uint32_t                                    viewCapacityInput,
	uint32_t*									viewCountOutput,
	XrViewConfigurationView*					views)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerEnumerateViewConfigurationViews(instance, systemId, viewConfigurationType, viewCapacityInput, viewCountOutput, views);
}

XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateSwapchainFormats(
	XrSession										session,
	uint32_t										formatCapacityInput,
	uint32_t*										formatCountOutput,
	int64_t*										formats)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerEnumerateSwapchainFormats(session, formatCapacityInput, formatCountOutput, formats);
}

XRAPI_ATTR XrResult XRAPI_CALL xrCreateSwapchain(
	XrSession										session,
	const XrSwapchainCreateInfo*					createInfo,
	XrSwapchain*									swapchain)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerCreateSwapchain(session, createInfo, swapchain);
}

XRAPI_ATTR XrResult XRAPI_CALL xrDestroySwapchain(
	XrSwapchain										swapchain)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerDestroySwapchain(swapchain);
}

XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateSwapchainImages(
	XrSwapchain                                 swapchain,
	uint32_t                                    imageCapacityInput,
	uint32_t*									imageCountOutput,
	XrSwapchainImageBaseHeader*					images)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerEnumerateSwapchainImages(swapchain, imageCapacityInput, imageCountOutput, images);
}

XRAPI_ATTR XrResult XRAPI_CALL xrAcquireSwapchainImage(
	XrSwapchain                                 swapchain,
	const XrSwapchainImageAcquireInfo*			acquireInfo,
	uint32_t*									index)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerAcquireSwapchainImage(swapchain, acquireInfo, index);
}

XRAPI_ATTR XrResult XRAPI_CALL xrWaitSwapchainImage(
	XrSwapchain                                 swapchain,
	const XrSwapchainImageWaitInfo*				waitInfo)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerWaitSwapchainImage(swapchain, waitInfo);
}

XRAPI_ATTR XrResult XRAPI_CALL xrReleaseSwapchainImage(
	XrSwapchain                                 swapchain,
	const XrSwapchainImageReleaseInfo*			releaseInfo)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerReleaseSwapchainImage(swapchain, releaseInfo);
}

XRAPI_ATTR XrResult XRAPI_CALL xrBeginSession(
	XrSession                                   session,
	const XrSessionBeginInfo*					beginInfo)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerBeginSession(session, beginInfo);
}

XRAPI_ATTR XrResult XRAPI_CALL xrEndSession(
	XrSession                                   session)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerEndSession(session);
}

XRAPI_ATTR XrResult XRAPI_CALL xrRequestExitSession(
	XrSession                                   session)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerRequestExitSession(session);
}

XRAPI_ATTR XrResult XRAPI_CALL xrWaitFrame(
	XrSession                                   session,
	const XrFrameWaitInfo*						frameWaitInfo,
	XrFrameState*								frameState)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerWaitFrame(session, frameWaitInfo, frameState);
}

XRAPI_ATTR XrResult XRAPI_CALL xrBeginFrame(
	XrSession                                   session,
	const XrFrameBeginInfo*						frameBeginInfo)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerBeginFrame(session, frameBeginInfo);
}

XRAPI_ATTR XrResult XRAPI_CALL xrEndFrame(
	XrSession                                   session,
	const XrFrameEndInfo*						frameEndInfo)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerEndFrame(session, frameEndInfo);
}

XRAPI_ATTR XrResult XRAPI_CALL xrLocateViews(
	XrSession                                   session,
	const XrViewLocateInfo*						viewLocateInfo,
	XrViewState*								viewState,
	uint32_t                                    viewCapacityInput,
	uint32_t*									viewCountOutput,
	XrView*										views)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerLocateViews(session, viewLocateInfo, viewState, viewCapacityInput, viewCountOutput, views);
}

XRAPI_ATTR XrResult XRAPI_CALL xrStringToPath(
	XrInstance                                  instance,
	const char*									pathString,
	XrPath*										path)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerStringToPath(instance, pathString, path);
}

XRAPI_ATTR XrResult XRAPI_CALL xrPathToString(
	XrInstance                                  instance,
	XrPath                                      path,
	uint32_t                                    bufferCapacityInput,
	uint32_t*									bufferCountOutput,
	char*										buffer)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerPathToString(instance, path, bufferCapacityInput, bufferCountOutput, buffer);
}

XRAPI_ATTR XrResult XRAPI_CALL xrCreateActionSet(
	XrInstance                                  instance,
	const XrActionSetCreateInfo*				createInfo,
	XrActionSet*								actionSet)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerCreateActionSet(instance, createInfo, actionSet);
}

XRAPI_ATTR XrResult XRAPI_CALL xrDestroyActionSet(
	XrActionSet                                 actionSet)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerDestroyActionSet(actionSet);
}

XRAPI_ATTR XrResult XRAPI_CALL xrCreateAction(
	XrActionSet                                 actionSet,
	const XrActionCreateInfo*					createInfo,
	XrAction*									action)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerCreateAction(actionSet, createInfo, action);
}

XRAPI_ATTR XrResult XRAPI_CALL xrDestroyAction(
	XrAction                                    action)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerDestroyAction(action);
}

XRAPI_ATTR XrResult XRAPI_CALL xrSuggestInteractionProfileBindings(
	XrInstance                                  instance,
	const XrInteractionProfileSuggestedBinding* suggestedBindings)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerSuggestInteractionProfileBindings(instance, suggestedBindings);
}

XRAPI_ATTR XrResult XRAPI_CALL xrAttachSessionActionSets(
	XrSession                                   session,
	const XrSessionActionSetsAttachInfo*		attachInfo)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerAttachSessionActionSets(session, attachInfo);
}

XRAPI_ATTR XrResult XRAPI_CALL xrGetCurrentInteractionProfile(
	XrSession                                   session,
	XrPath                                      topLevelUserPath,
	XrInteractionProfileState*					interactionProfile)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerGetCurrentInteractionProfile(session, topLevelUserPath, interactionProfile);
}

XRAPI_ATTR XrResult XRAPI_CALL xrGetActionStateBoolean(
	XrSession                                   session,
	const XrActionStateGetInfo*					getInfo,
	XrActionStateBoolean*						state)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerGetActionStateBoolean(session, getInfo, state);
}

XRAPI_ATTR XrResult XRAPI_CALL xrGetActionStateFloat(
	XrSession                                   session,
	const XrActionStateGetInfo*					getInfo,
	XrActionStateFloat*							state)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerGetActionStateFloat(session, getInfo, state);
}

XRAPI_ATTR XrResult XRAPI_CALL xrGetActionStateVector2f(
	XrSession                                   session,
	const XrActionStateGetInfo*					getInfo,
	XrActionStateVector2f*						state)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerGetActionStateVector2f(session, getInfo, state);
}

XRAPI_ATTR XrResult XRAPI_CALL xrGetActionStatePose(
	XrSession                                   session,
	const XrActionStateGetInfo*					getInfo,
	XrActionStatePose*							state)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerGetActionStatePose(session, getInfo, state);
}

XRAPI_ATTR XrResult XRAPI_CALL xrSyncActions(
	XrSession                                   session,
	const										XrActionsSyncInfo* syncInfo)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerSyncActions(session, syncInfo);
}

XRAPI_ATTR XrResult XRAPI_CALL xrEnumerateBoundSourcesForAction(
	XrSession                                   session,
	const XrBoundSourcesForActionEnumerateInfo* enumerateInfo,
	uint32_t                                    sourceCapacityInput,
	uint32_t*									sourceCountOutput,
	XrPath*										sources)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerEnumerateBoundSourcesForAction(session, enumerateInfo, sourceCapacityInput, sourceCountOutput, sources);
}

XRAPI_ATTR XrResult XRAPI_CALL xrGetInputSourceLocalizedName(
	XrSession                                   session,
	const XrInputSourceLocalizedNameGetInfo*	getInfo,
	uint32_t                                    bufferCapacityInput,
	uint32_t*									bufferCountOutput,
	char*										buffer)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerGetInputSourceLocalizedName(session, getInfo, bufferCapacityInput, bufferCountOutput, buffer);
}

XRAPI_ATTR XrResult XRAPI_CALL xrApplyHapticFeedback(
	XrSession                                   session,
	const XrHapticActionInfo*					hapticActionInfo,
	const XrHapticBaseHeader*					hapticFeedback)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerApplyHapticFeedback(session, hapticActionInfo, hapticFeedback);
}

XRAPI_ATTR XrResult XRAPI_CALL xrStopHapticFeedback(
	XrSession                                   session,
	const XrHapticActionInfo*					hapticActionInfo)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerStopHapticFeedback(session, hapticActionInfo);
}

// Global extension Functions

// XR_KHR_loader_init
XRAPI_ATTR XrResult XRAPI_CALL xrInitializeLoaderKHR(
	const XrLoaderInitInfoBaseHeaderKHR* loaderInitInfo)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerInitializeLoaderKHR(loaderInitInfo);
}

// Instance extension functions

// XR_KHR_visibility_mask
XRAPI_ATTR XrResult XRAPI_CALL xrGetVisibilityMaskKHR(
	XrSession session,
	XrViewConfigurationType viewConfigurationType,
	uint32_t viewIndex,
	XrVisibilityMaskTypeKHR visibilityMaskType,
	XrVisibilityMaskKHR* visibilityMask)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerGetVisibilityMaskKHR(session, viewConfigurationType, viewIndex, visibilityMaskType, visibilityMask);
}

#ifdef XR_USE_GRAPHICS_API_D3D11
// XR_KHR_D3D11_enable
XRAPI_ATTR XrResult XRAPI_CALL xrGetD3D11GraphicsRequirementsKHR(
	XrInstance                                  instance,
	XrSystemId                                  systemId,
	XrGraphicsRequirementsD3D11KHR* graphicsRequirements)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerGetD3D11GraphicsRequirementsKHR(instance, systemId, graphicsRequirements);
}
#endif

#ifdef XR_USE_GRAPHICS_API_D3D12
// XR_KHR_D3D12_enable
XRAPI_ATTR XrResult XRAPI_CALL xrGetD3D12GraphicsRequirementsKHR(
	XrInstance                                  instance,
	XrSystemId                                  systemId,
	XrGraphicsRequirementsD3D12KHR* graphicsRequirements)
{
	IOpenXRAPILayerManager& APISurface = IOpenXRAPILayerManager::Get();
	return APISurface.GetActiveLayer()->XrLayerGetD3D12GraphicsRequirementsKHR(instance, systemId, graphicsRequirements);
}
#endif

// I could use C-style cast, but I wanted to use the C++ pattern. Couldn't cast directly to PFN_xrVoidFunction, hence the double reinterpret_cast.
// I could also use PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS
#define XRSCRIBE_MAP_GLOBAL_FUNCTION(Type,Func) UE::XRScribe::GlobalFunctionMap.Add(FString(TEXT(#Func)), reinterpret_cast<PFN_xrVoidFunction>(reinterpret_cast<void*>(UE::XRScribe::Func)));
#define XRSCRIBE_MAP_INSTANCE_FUNCTION(Type,Func) UE::XRScribe::InstanceFunctionMap.Add(FString(TEXT(#Func)), reinterpret_cast<PFN_xrVoidFunction>(reinterpret_cast<void*>(UE::XRScribe::Func)));

#define ENUM_XR_ENTRYPOINTS_GLOBAL_EXTENSION(EnumMacro) \
	EnumMacro(PFN_xrInitializeLoaderKHR,xrInitializeLoaderKHR)

// TODO: list unused functions?

//#define ENUM_XR_ENTRYPOINTS_INSTANCE_EXTENSION(EnumMacro) \
//	EnumMacro(PFN_xrGetD3D11GraphicsRequirementsKHR,xrGetD3D11GraphicsRequirementsKHR) \
//	EnumMacro(PFN_xrGetD3D12GraphicsRequirementsKHR,xrGetD3D12GraphicsRequirementsKHR)
	//EnumMacro(PFN_xrGetVisibilityMaskKHR, xrGetVisibilityMaskKHR) \
	//EnumMacro(PFN_xrCreateHandTrackerEXT,xrCreateHandTrackerEXT) \
	//EnumMacro(PFN_xrDestroyHandTrackerEXT,xrDestroyHandTrackerEXT) \
	//EnumMacro(PFN_xrLocateHandJointsEXT,xrLocateHandJointsEXT) \
	//EnumMacro(PFN_xrGetOpenGLGraphicsRequirementsKHR,xrGetOpenGLGraphicsRequirementsKHR) \
	//EnumMacro(PFN_xrGetOpenGLESGraphicsRequirementsKHR,xrGetOpenGLESGraphicsRequirementsKHR) \
	//EnumMacro(PFN_xrGetVulkanInstanceExtensionsKHR,xrGetVulkanInstanceExtensionsKHR) \
	//EnumMacro(PFN_xrGetVulkanDeviceExtensionsKHR,xrGetVulkanDeviceExtensionsKHR) \
	//EnumMacro(PFN_xrGetVulkanGraphicsRequirementsKHR,xrGetVulkanGraphicsRequirementsKHR) \
	//EnumMacro(PFN_xrConvertWin32PerformanceCounterToTimeKHR,xrConvertWin32PerformanceCounterToTimeKHR)

void BuildFunctionMaps()
{
	ENUM_XR_ENTRYPOINTS_GLOBAL(XRSCRIBE_MAP_GLOBAL_FUNCTION);
	ENUM_XR_ENTRYPOINTS(XRSCRIBE_MAP_INSTANCE_FUNCTION);

	// Global extension functions
	ENUM_XR_ENTRYPOINTS_GLOBAL_EXTENSION(XRSCRIBE_MAP_GLOBAL_FUNCTION);
	//ENUM_XR_ENTRYPOINTS_INSTANCE_EXTENSION(XRSCRIBE_MAP_INSTANCE_FUNCTION);

	UE::XRScribe::InstanceFunctionMap.Add(FString(TEXT("xrGetVisibilityMaskKHR")), reinterpret_cast<PFN_xrVoidFunction>(reinterpret_cast<void*>(UE::XRScribe::xrGetVisibilityMaskKHR)));
	UE::XRScribe::ExtensionFunctionGateMap.Add(FString(TEXT("xrGetVisibilityMaskKHR")), FString(TEXT("XR_KHR_visibility_mask")));

#ifdef XR_USE_GRAPHICS_API_D3D11
	UE::XRScribe::InstanceFunctionMap.Add(FString(TEXT("xrGetD3D11GraphicsRequirementsKHR")), reinterpret_cast<PFN_xrVoidFunction>(reinterpret_cast<void*>(UE::XRScribe::xrGetD3D11GraphicsRequirementsKHR)));
	UE::XRScribe::ExtensionFunctionGateMap.Add(FString(TEXT("xrGetD3D11GraphicsRequirementsKHR")), FString(TEXT("XR_KHR_D3D11_enable")));
#endif

#ifdef XR_USE_GRAPHICS_API_D3D12
	UE::XRScribe::InstanceFunctionMap.Add(FString(TEXT("xrGetD3D12GraphicsRequirementsKHR")), reinterpret_cast<PFN_xrVoidFunction>(reinterpret_cast<void*>(UE::XRScribe::xrGetD3D12GraphicsRequirementsKHR)));
	UE::XRScribe::ExtensionFunctionGateMap.Add(FString(TEXT("xrGetD3D12GraphicsRequirementsKHR")), FString(TEXT("XR_KHR_D3D12_enable")));
#endif

}

#undef XRSCRIBE_MAP_INSTANCE_FUNCTION
#undef XRSCRIBE_MAP_GLOBAL_FUNCTION

void ClearFunctionMaps()
{
	GlobalFunctionMap.Reset();
	InstanceFunctionMap.Reset();
}

EXRScribeRunMode DetermineRunMode(EXRScribeRunMode FallbackRunMode, FString& EmulationLoadPath)
{
	// TODO: load file from plugin settings

	EXRScribeRunMode RunModeFromConfig = EXRScribeRunMode::Emulate;
	const bool bModeReadFromConfig = GConfig->GetInt(TEXT("/Script/XRScribe.XRScribeDeveloperSettings"), TEXT("XRScribe.RunMode"), (int32&)RunModeFromConfig, GEngineIni);
	if (!bModeReadFromConfig)
	{
		RunModeFromConfig = FallbackRunMode;
	}

	bool bEmulationCLI = false;
	EmulationLoadPath = FPaths::ProjectSavedDir() / TEXT("Capture.xrs");

	FString RequestedEmulationProfile;
	bEmulationCLI = FParse::Value(FCommandLine::Get(), TEXT("-xremu="), RequestedEmulationProfile);
	if (bEmulationCLI)
	{
		UE_LOGF(LogXRScribeAPI, Log, "xremu command-line argument detected to emulate built-in capture");

		FString ExtrasDir = IPluginManager::Get().FindPlugin("XRScribe")->GetBaseDir() / TEXT("Extras");
		if (RequestedEmulationProfile == TEXT("mq2"))
		{
			EmulationLoadPath = ExtrasDir / TEXT("mq2.xrs");
			UE_LOGF(LogXRScribeAPI, Log, "mq2 built-in capture requested");
		}
		else if (RequestedEmulationProfile == TEXT("vi"))
		{
			EmulationLoadPath = ExtrasDir / TEXT("vi.xrs");
			UE_LOGF(LogXRScribeAPI, Log, "vi built-in capture requested");
		}
		else
		{
			UE_LOGF(LogXRScribeAPI, Warning, "Unknown built-in capture, deferring to config run-mode");
			bEmulationCLI = false;
		}
	}

	return bEmulationCLI ? EXRScribeRunMode::Emulate : RunModeFromConfig;
}

class FOpenXRAPILayerManager : public IOpenXRAPILayerManager
{
public:
	FOpenXRAPILayerManager()
	{
		CaptureLayer = MakeShared<FOpenXRCaptureLayer, ESPMode::ThreadSafe>();
		EmulationLayer = MakeShared<FOpenXREmulationLayer, ESPMode::ThreadSafe>();
	}

	virtual ~FOpenXRAPILayerManager() {}

	IOpenXRAPILayer* GetActiveLayer() const override
	{
		return ActiveLayer.Get();
	}

	void SetFallbackRunMode(int32 Fallback) override
	{
		if ((Fallback >= (int32)EXRScribeRunMode::Capture) &&
			(Fallback <= (int32)EXRScribeRunMode::Emulate))
		{
			FallbackRunMode = (EXRScribeRunMode)Fallback;
		}
		else
		{
			UE_LOGF(LogXRScribeAPI, Warning, "Invalid run mode override specified (%d), ignoring", Fallback);
		}
	}

	bool SetChainedGetProcAddr(PFN_xrGetInstanceProcAddr InChainedGetProcAddr) override
	{
		ChainedGetProcAddr = InChainedGetProcAddr;

		FString EmulationLoadPath;
		const EXRScribeRunMode SelectedRunMode = DetermineRunMode(FallbackRunMode, EmulationLoadPath);

		if (SelectedRunMode == EXRScribeRunMode::Capture)
		{
			UE_LOGF(LogXRScribeAPI, Log, "Capture layer selected");

			if (InChainedGetProcAddr != nullptr)
			{
				UE_LOGF(LogXRScribeAPI, Log, "Valid PFN_xrGetInstanceProcAddr handed off to Capture Layer");
				ActiveLayer = CaptureLayer;
				CaptureLayer->SetChainedGetProcAddr(ChainedGetProcAddr);
				return true;
			}
			else
			{
				UE_LOGF(LogXRScribeAPI, Warning, "Invalid PFN_xrGetInstanceProcAddr, Capture Layer disabled");
				return false;
			}
		}
		else if (SelectedRunMode == EXRScribeRunMode::Emulate)
		{
			UE_LOGF(LogXRScribeAPI, Log, "Emulation layer selected");
			ActiveLayer = EmulationLayer;
			const bool bEmulationLoadedCapture = reinterpret_cast<FOpenXREmulationLayer*>(EmulationLayer.Get())->LoadCaptureFromFile(EmulationLoadPath);

			if (bEmulationLoadedCapture)
			{
				UE_LOGF(LogXRScribeAPI, Log, "Emulation Layer enabled");
			}
			else
			{
				UE_LOGF(LogXRScribeAPI, Log, "Emulation Layer disabled - unable to load the capture file.");
			}

			return bEmulationLoadedCapture;
		}
		else
		{
			check(0);
		}
		return false;
	}

private:
	TSharedPtr<IOpenXRAPILayer, ESPMode::ThreadSafe> ActiveLayer;
	TSharedPtr<IOpenXRAPILayer, ESPMode::ThreadSafe> CaptureLayer;
	TSharedPtr<IOpenXRAPILayer, ESPMode::ThreadSafe> EmulationLayer;

	PFN_xrGetInstanceProcAddr ChainedGetProcAddr = nullptr;

	EXRScribeRunMode FallbackRunMode = EXRScribeRunMode::Emulate;
};

static FOpenXRAPILayerManager OpenXRAPISurface;

IOpenXRAPILayerManager& IOpenXRAPILayerManager::Get()
{
	return OpenXRAPISurface;
}

} // namespace UE::XRScribe
