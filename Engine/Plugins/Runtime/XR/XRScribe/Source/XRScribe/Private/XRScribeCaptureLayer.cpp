// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRScribeCaptureLayer.h"

#include "Containers/StringConv.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"

DEFINE_LOG_CATEGORY_STATIC(LogXRScribeCapture, Log, All);

namespace UE::XRScribe
{

void FOpenXRCaptureLayer::SaveCaptureToFile()
{
	// TODO: source path from config?
	const FString CaptureFilePath = FPaths::ProjectSavedDir() / TEXT("Capture.xrs");
	TArrayView<const uint8> SaveDataView(CaptureEncoder.GetData(), CaptureEncoder.Num());
	const bool FileSaveResult = FFileHelper::SaveArrayToFile(SaveDataView, *CaptureFilePath);

	if (FileSaveResult)
	{
		UE_LOGF(LogXRScribeCapture, Log, "Capture file saved to disk: %ls", *CaptureFilePath);

	}
	else
	{
		UE_LOGF(LogXRScribeCapture, Error, "Failed to save capture file to disk");
	}
}

FOpenXRCaptureLayer::FOpenXRCaptureLayer()
{
	NativeSupportedExtensionNames.Add(ANSI_TO_TCHAR(XR_KHR_VISIBILITY_MASK_EXTENSION_NAME));
#if defined(XR_USE_GRAPHICS_API_D3D11)
	NativeSupportedExtensionNames.Add(ANSI_TO_TCHAR(XR_KHR_D3D11_ENABLE_EXTENSION_NAME));
#endif
#if defined(XR_USE_GRAPHICS_API_D3D12)
	NativeSupportedExtensionNames.Add(ANSI_TO_TCHAR(XR_KHR_D3D12_ENABLE_EXTENSION_NAME));
#endif
}

FOpenXRCaptureLayer::~FOpenXRCaptureLayer()
{
}

static void GetProcAddrHelper(PFN_xrGetInstanceProcAddr GetProcAddr, XrInstance Instance, const char* FuncName, PFN_xrVoidFunction* FuncPtr)
{
	XrResult GetResult = GetProcAddr(Instance, FuncName, FuncPtr);
	
	if (GetResult != XrResult::XR_SUCCESS)
	{
		UE_LOGF(LogXRScribeCapture, Warning, "Failed to obtain proc address for %ls", ANSI_TO_TCHAR(FuncName));
	}
}

void FOpenXRCaptureLayer::SetChainedGetProcAddr(PFN_xrGetInstanceProcAddr InChainedGetProcAddr)
{
	check(InChainedGetProcAddr != nullptr);
	FunctionPassthroughs.GetInstanceProcAddr = InChainedGetProcAddr;

	GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, XR_NULL_HANDLE, "xrEnumerateApiLayerProperties", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.EnumerateApiLayerProperties));
	GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, XR_NULL_HANDLE, "xrEnumerateInstanceExtensionProperties", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.EnumerateInstanceExtensionProperties));
	GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, XR_NULL_HANDLE, "xrCreateInstance", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.CreateInstance));
	
	// TODO: unsure when to actually try to hook this
	//GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, XR_NULL_HANDLE, "xrInitializeLoaderKHR", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.InitializeLoaderKHR));

	{
		uint32_t NumExtensions = 0;
		XrResult Result = FunctionPassthroughs.EnumerateInstanceExtensionProperties(nullptr, 0, &NumExtensions, nullptr);
		ensure(Result == XR_SUCCESS);
		CaptureEncoder.EncodeEnumerateInstanceExtensionProperties(Result, nullptr, 0, &NumExtensions, nullptr);


		TArray<XrExtensionProperties> TotalExtensionProperties;
		TotalExtensionProperties.AddZeroed(NumExtensions);
		for (XrExtensionProperties& Property : TotalExtensionProperties)
		{
			Property.type = XR_TYPE_EXTENSION_PROPERTIES;
		}
		
		Result = FunctionPassthroughs.EnumerateInstanceExtensionProperties(nullptr, NumExtensions, &NumExtensions, TotalExtensionProperties.GetData());
		ensure(Result == XR_SUCCESS);
		CaptureEncoder.EncodeEnumerateInstanceExtensionProperties(Result, nullptr, NumExtensions, &NumExtensions, TotalExtensionProperties.GetData());

		for (const XrExtensionProperties& RuntimeExtensionProperty : TotalExtensionProperties)
		{
			if (NativeSupportedExtensionNames.Contains(FString(ANSI_TO_TCHAR(RuntimeExtensionProperty.extensionName))))
			{
				PassthruInstanceSupportedExtensions.Add(RuntimeExtensionProperty);
			}
		}
	}
}

bool FOpenXRCaptureLayer::SupportsInstanceExtension(const ANSICHAR* ExtensionName)
{
	return CaptureActiveExtensionNames.Contains(FString(ANSI_TO_TCHAR(ExtensionName)));
}

// Global
XrResult FOpenXRCaptureLayer::XrLayerEnumerateApiLayerProperties(uint32_t propertyCapacityInput, uint32_t* propertyCountOutput, XrApiLayerProperties* properties)
{
	const XrResult Result = FunctionPassthroughs.EnumerateApiLayerProperties(propertyCapacityInput, propertyCountOutput, properties);
	CaptureEncoder.EncodeEnumerateApiLayerProperties(Result, propertyCapacityInput, propertyCountOutput, properties);
	return Result;
}

XrResult FOpenXRCaptureLayer::XrLayerEnumerateInstanceExtensionProperties(const char* layerName, uint32_t propertyCapacityInput, uint32_t* propertyCountOutput, XrExtensionProperties* properties)
{
	if (layerName != nullptr)
	{
		UE_LOGF(LogXRScribeCapture, Error, "Capture layer does not support layer-specific extensions");
		return XR_ERROR_RUNTIME_FAILURE;
	}

	if (propertyCapacityInput == 0)
	{
		if (propertyCountOutput == nullptr)
		{
			return XR_ERROR_VALIDATION_FAILURE;
		}
		*propertyCountOutput = PassthruInstanceSupportedExtensions.Num();
	}
	else
	{
		if ((propertyCountOutput == nullptr) ||
			(properties == nullptr))
		{
			return XR_ERROR_VALIDATION_FAILURE;
		}
		if (propertyCapacityInput < (uint32)PassthruInstanceSupportedExtensions.Num())
		{
			return XR_ERROR_SIZE_INSUFFICIENT;
		}

		const uint32 NumPropertiesToCopy = PassthruInstanceSupportedExtensions.Num();
		*propertyCountOutput = NumPropertiesToCopy;

		FMemory::Memcpy(properties, PassthruInstanceSupportedExtensions.GetData(), NumPropertiesToCopy * sizeof(XrExtensionProperties));

	}

	return XR_SUCCESS;
}

XrResult FOpenXRCaptureLayer::XrLayerCreateInstance(const XrInstanceCreateInfo* createInfo, XrInstance* instance)
{
	const XrResult Result = FunctionPassthroughs.CreateInstance(createInfo, instance);
	CaptureEncoder.EncodeCreateInstance(Result, createInfo, instance);
	
	if (Result == XR_SUCCESS)
	{
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrDestroyInstance", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.DestroyInstance));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrGetInstanceProperties", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.GetInstanceProperties));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrPollEvent", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.PollEvent));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrResultToString", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.ResultToString));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrStructureTypeToString", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.StructureTypeToString));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrGetSystem", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.GetSystem));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrGetSystemProperties", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.GetSystemProperties));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrEnumerateEnvironmentBlendModes", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.EnumerateEnvironmentBlendModes));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrCreateSession", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.CreateSession));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrDestroySession", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.DestroySession));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrEnumerateReferenceSpaces", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.EnumerateReferenceSpaces));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrCreateReferenceSpace", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.CreateReferenceSpace));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrGetReferenceSpaceBoundsRect", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.GetReferenceSpaceBoundsRect));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrCreateActionSpace", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.CreateActionSpace));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrLocateSpace", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.LocateSpace));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrDestroySpace", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.DestroySpace));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrEnumerateViewConfigurations", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.EnumerateViewConfigurations));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrGetViewConfigurationProperties", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.GetViewConfigurationProperties));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrEnumerateViewConfigurationViews", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.EnumerateViewConfigurationViews));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrEnumerateSwapchainFormats", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.EnumerateSwapchainFormats));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrCreateSwapchain", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.CreateSwapchain));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrDestroySwapchain", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.DestroySwapchain));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrEnumerateSwapchainImages", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.EnumerateSwapchainImages));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrAcquireSwapchainImage", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.AcquireSwapchainImage));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrWaitSwapchainImage", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.WaitSwapchainImage));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrReleaseSwapchainImage", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.ReleaseSwapchainImage));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrBeginSession", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.BeginSession));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrEndSession", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.EndSession));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrRequestExitSession", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.RequestExitSession));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrWaitFrame", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.WaitFrame));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrBeginFrame", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.BeginFrame));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrEndFrame", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.EndFrame));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrLocateViews", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.LocateViews));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrStringToPath", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.StringToPath));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrPathToString", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.PathToString));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrCreateActionSet", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.CreateActionSet));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrDestroyActionSet", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.DestroyActionSet));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrCreateAction", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.CreateAction));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrDestroyAction", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.DestroyAction));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrSuggestInteractionProfileBindings", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.SuggestInteractionProfileBindings));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrAttachSessionActionSets", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.AttachSessionActionSets));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrGetCurrentInteractionProfile", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.GetCurrentInteractionProfile));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrGetActionStateBoolean", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.GetActionStateBoolean));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrGetActionStateFloat", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.GetActionStateFloat));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrGetActionStateVector2f", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.GetActionStateVector2f));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrGetActionStatePose", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.GetActionStatePose));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrSyncActions", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.SyncActions));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrEnumerateBoundSourcesForAction", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.EnumerateBoundSourcesForAction));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrGetInputSourceLocalizedName", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.GetInputSourceLocalizedName));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrApplyHapticFeedback", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.ApplyHapticFeedback));
		GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrStopHapticFeedback", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.StopHapticFeedback));

		// Extensions are managed different than core functions
		if (createInfo->enabledExtensionCount > 0)
		{
			CaptureActiveExtensionNames.Reserve(createInfo->enabledExtensionCount);

			// We could save the full extension info, but the names are fine for now
			for (uint32 ExtensionIndex = 0; ExtensionIndex < createInfo->enabledExtensionCount; ExtensionIndex++)
			{
				CaptureActiveExtensionNames.Add(ANSI_TO_TCHAR(createInfo->enabledExtensionNames[ExtensionIndex]));
			}

			if (CaptureActiveExtensionNames.Contains(XR_KHR_VISIBILITY_MASK_EXTENSION_NAME))
			{
				GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrGetVisibilityMaskKHR", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.GetVisibilityMaskKHR));
			}
#if defined(XR_USE_GRAPHICS_API_D3D11)
			if (CaptureActiveExtensionNames.Contains(XR_KHR_D3D11_ENABLE_EXTENSION_NAME))
			{
				GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrGetD3D11GraphicsRequirementsKHR", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.GetD3D11GraphicsRequirementsKHR));
			}
#endif
#if defined(XR_USE_GRAPHICS_API_D3D12)
			if (CaptureActiveExtensionNames.Contains(XR_KHR_D3D12_ENABLE_EXTENSION_NAME))
			{
				GetProcAddrHelper(FunctionPassthroughs.GetInstanceProcAddr, *instance, "xrGetD3D12GraphicsRequirementsKHR", reinterpret_cast<PFN_xrVoidFunction*>(&FunctionPassthroughs.GetD3D12GraphicsRequirementsKHR));
			}
#endif
		}
	}

	return Result;
}

// Instance
XrResult FOpenXRCaptureLayer::XrLayerDestroyInstance(XrInstance instance)
{
	const XrResult Result = FunctionPassthroughs.DestroyInstance(instance);
	CaptureEncoder.EncodeDestroyInstance(Result, instance);
	return Result;
}

XrResult FOpenXRCaptureLayer::XrLayerGetInstanceProperties(XrInstance instance, XrInstanceProperties* instanceProperties)
{
	const XrResult Result = FunctionPassthroughs.GetInstanceProperties(instance, instanceProperties);
	CaptureEncoder.EncodeGetInstanceProperties(Result, instance, instanceProperties);
	return Result;
}

XrResult FOpenXRCaptureLayer::XrLayerPollEvent(XrInstance instance, XrEventDataBuffer* eventData)
{
	const XrResult Result = FunctionPassthroughs.PollEvent(instance, eventData);
	CaptureEncoder.EncodePollEvent(Result, instance, eventData);
	return Result;
}

XrResult FOpenXRCaptureLayer::XrLayerResultToString(XrInstance instance, XrResult value, char buffer[XR_MAX_RESULT_STRING_SIZE])
{ 
	const XrResult Result = FunctionPassthroughs.ResultToString(instance, value, buffer);
	CaptureEncoder.EncodeResultToString(Result, instance, value, buffer);
	return Result;
}

XrResult FOpenXRCaptureLayer::XrLayerStructureTypeToString(XrInstance instance, XrStructureType value, char buffer[XR_MAX_STRUCTURE_NAME_SIZE])
{
	const XrResult Result = FunctionPassthroughs.StructureTypeToString(instance, value, buffer);
	CaptureEncoder.EncodeStructureTypeToString(Result, instance, value, buffer);
	return Result;
}

XrResult FOpenXRCaptureLayer::XrLayerGetSystem(XrInstance instance, const XrSystemGetInfo* getInfo, XrSystemId* systemId)
{
	const XrResult Result = FunctionPassthroughs.GetSystem(instance, getInfo, systemId);
	CaptureEncoder.EncodeGetSystem(Result, instance, getInfo, systemId);
	return Result;

	// TODO: might want to fetch the system properties here in case UE doesn't
}

XrResult FOpenXRCaptureLayer::XrLayerGetSystemProperties(XrInstance instance, XrSystemId systemId, XrSystemProperties* properties)
{
	const XrResult Result = FunctionPassthroughs.GetSystemProperties(instance, systemId, properties);
	CaptureEncoder.EncodeGetSystemProperties(Result, instance, systemId, properties);
	return Result;
}

XrResult FOpenXRCaptureLayer::XrLayerEnumerateEnvironmentBlendModes(XrInstance instance, XrSystemId systemId, XrViewConfigurationType viewConfigurationType, uint32_t environmentBlendModeCapacityInput, uint32_t* environmentBlendModeCountOutput, XrEnvironmentBlendMode* environmentBlendModes)
{
	const XrResult Result = FunctionPassthroughs.EnumerateEnvironmentBlendModes(instance, systemId, viewConfigurationType, environmentBlendModeCapacityInput, environmentBlendModeCountOutput, environmentBlendModes);
	CaptureEncoder.EncodeEnumerateEnvironmentBlendModes(Result, instance, systemId, viewConfigurationType, environmentBlendModeCapacityInput, environmentBlendModeCountOutput, environmentBlendModes);
	return Result;
}

XrResult FOpenXRCaptureLayer::XrLayerCreateSession(XrInstance instance, const XrSessionCreateInfo* createInfo, XrSession* session)
{
	const XrResult Result = FunctionPassthroughs.CreateSession(instance, createInfo, session);
	CaptureEncoder.EncodeCreateSession(Result, instance, createInfo, session);
	return Result;
}

XrResult FOpenXRCaptureLayer::XrLayerDestroySession(XrSession session)
{
	const XrResult Result = FunctionPassthroughs.DestroySession(session);
	CaptureEncoder.EncodeDestroySession(Result, session);
	return Result;
}

XrResult FOpenXRCaptureLayer::XrLayerEnumerateReferenceSpaces(XrSession session, uint32_t spaceCapacityInput, uint32_t* spaceCountOutput, XrReferenceSpaceType* spaces)
{
	const XrResult Result = FunctionPassthroughs.EnumerateReferenceSpaces(session, spaceCapacityInput, spaceCountOutput, spaces);
	CaptureEncoder.EncodeEnumerateReferenceSpaces(Result, session, spaceCapacityInput, spaceCountOutput, spaces);
	return Result;
}

XrResult FOpenXRCaptureLayer::XrLayerCreateReferenceSpace(XrSession session, const XrReferenceSpaceCreateInfo* createInfo, XrSpace* space)
{
	const XrResult Result = FunctionPassthroughs.CreateReferenceSpace(session, createInfo, space);
	CaptureEncoder.EncodeCreateReferenceSpace(Result, session, createInfo, space);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerGetReferenceSpaceBoundsRect(XrSession session, XrReferenceSpaceType referenceSpaceType, XrExtent2Df* bounds)
{
	const XrResult Result = FunctionPassthroughs.GetReferenceSpaceBoundsRect(session, referenceSpaceType, bounds);
	CaptureEncoder.EncodeGetReferenceSpaceBoundsRect(Result, session, referenceSpaceType, bounds);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerCreateActionSpace(XrSession session, const XrActionSpaceCreateInfo* createInfo, XrSpace* space)
{
	const XrResult Result = FunctionPassthroughs.CreateActionSpace(session, createInfo, space);
	CaptureEncoder.EncodeCreateActionSpace(Result, session, createInfo, space);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerLocateSpace(XrSpace space, XrSpace baseSpace, XrTime time, XrSpaceLocation* location)
{
	const XrResult Result = FunctionPassthroughs.LocateSpace(space, baseSpace, time, location);
	CaptureEncoder.EncodeLocateSpace(Result, space, baseSpace, time, location);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerDestroySpace(XrSpace space)
{
	const XrResult Result = FunctionPassthroughs.DestroySpace(space);
	CaptureEncoder.EncodeDestroySpace(Result, space);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerEnumerateViewConfigurations(XrInstance instance, XrSystemId systemId, uint32_t viewConfigurationTypeCapacityInput, uint32_t* viewConfigurationTypeCountOutput, XrViewConfigurationType* viewConfigurationTypes)
{
	const XrResult Result = FunctionPassthroughs.EnumerateViewConfigurations(instance, systemId, viewConfigurationTypeCapacityInput, viewConfigurationTypeCountOutput, viewConfigurationTypes);
	CaptureEncoder.EncodeEnumerateViewConfigurations(Result, instance, systemId, viewConfigurationTypeCapacityInput, viewConfigurationTypeCountOutput, viewConfigurationTypes);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerGetViewConfigurationProperties(XrInstance instance, XrSystemId systemId, XrViewConfigurationType viewConfigurationType, XrViewConfigurationProperties* configurationProperties)
{
	// TODO: in the future, we should just fetch properties for all supported XrViewConfigurationType upfront
	const XrResult Result = FunctionPassthroughs.GetViewConfigurationProperties(instance, systemId, viewConfigurationType, configurationProperties);
	CaptureEncoder.EncodeGetViewConfigurationProperties(Result, instance, systemId, viewConfigurationType, configurationProperties);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerEnumerateViewConfigurationViews(XrInstance instance, XrSystemId systemId, XrViewConfigurationType viewConfigurationType, uint32_t viewCapacityInput, uint32_t* viewCountOutput, XrViewConfigurationView* views)
{
	const XrResult Result = FunctionPassthroughs.EnumerateViewConfigurationViews(instance, systemId, viewConfigurationType, viewCapacityInput, viewCountOutput, views);
	CaptureEncoder.EncodeEnumerateViewConfigurationViews(Result, instance, systemId, viewConfigurationType, viewCapacityInput, viewCountOutput, views);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerEnumerateSwapchainFormats(XrSession session, uint32_t formatCapacityInput, uint32_t* formatCountOutput, int64_t* formats)
{
	const XrResult Result = FunctionPassthroughs.EnumerateSwapchainFormats(session, formatCapacityInput, formatCountOutput, formats);
	CaptureEncoder.EncodeEnumerateSwapchainFormats(Result, session, formatCapacityInput, formatCountOutput, formats);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerCreateSwapchain(XrSession session, const XrSwapchainCreateInfo* createInfo, XrSwapchain* swapchain)
{
	const XrResult Result = FunctionPassthroughs.CreateSwapchain(session, createInfo, swapchain);
	CaptureEncoder.EncodeCreateSwapchain(Result, session, createInfo, swapchain);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerDestroySwapchain(XrSwapchain swapchain)
{
	const XrResult Result = FunctionPassthroughs.DestroySwapchain(swapchain);
	CaptureEncoder.EncodeDestroySwapchain(Result, swapchain);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerEnumerateSwapchainImages(XrSwapchain swapchain, uint32_t imageCapacityInput, uint32_t* imageCountOutput, XrSwapchainImageBaseHeader* images)
{
	const XrResult Result = FunctionPassthroughs.EnumerateSwapchainImages(swapchain, imageCapacityInput, imageCountOutput, images);
	CaptureEncoder.EncodeEnumerateSwapchainImages(Result, swapchain, imageCapacityInput, imageCountOutput, images);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerAcquireSwapchainImage(XrSwapchain swapchain, const XrSwapchainImageAcquireInfo* acquireInfo, uint32_t* index)
{
	const XrResult Result = FunctionPassthroughs.AcquireSwapchainImage(swapchain, acquireInfo, index);
	CaptureEncoder.EncodeAcquireSwapchainImage(Result, swapchain, acquireInfo, index);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerWaitSwapchainImage(XrSwapchain swapchain, const XrSwapchainImageWaitInfo* waitInfo)
{
	const XrResult Result = FunctionPassthroughs.WaitSwapchainImage(swapchain, waitInfo);
	CaptureEncoder.EncodeWaitSwapchainImage(Result, swapchain, waitInfo);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerReleaseSwapchainImage(XrSwapchain swapchain, const XrSwapchainImageReleaseInfo* releaseInfo)
{
	const XrResult Result = FunctionPassthroughs.ReleaseSwapchainImage(swapchain, releaseInfo);
	CaptureEncoder.EncodeReleaseSwapchainImage(Result, swapchain, releaseInfo);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerBeginSession(XrSession session, const XrSessionBeginInfo* beginInfo)
{
	const XrResult Result = FunctionPassthroughs.BeginSession(session, beginInfo);
	CaptureEncoder.EncodeBeginSession(Result, session, beginInfo);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerEndSession(XrSession session)
{
	const XrResult Result = FunctionPassthroughs.EndSession(session);
	CaptureEncoder.EncodeEndSession(Result, session);

	// Capturing one session is enough signal to save off
	// TODO: Make this configurable
	SaveCaptureToFile();

	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerRequestExitSession(XrSession session)
{
	const XrResult Result = FunctionPassthroughs.RequestExitSession(session);
	CaptureEncoder.EncodeRequestExitSession(Result, session);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerWaitFrame(XrSession session, const XrFrameWaitInfo* frameWaitInfo, XrFrameState* frameState)
{
	const XrResult Result = FunctionPassthroughs.WaitFrame(session, frameWaitInfo, frameState);
	CaptureEncoder.EncodeWaitFrame(Result, session, frameWaitInfo, frameState);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerBeginFrame(XrSession session, const XrFrameBeginInfo* frameBeginInfo)
{
	const XrResult Result = FunctionPassthroughs.BeginFrame(session, frameBeginInfo);
	CaptureEncoder.EncodeBeginFrame(Result, session, frameBeginInfo);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerEndFrame(XrSession session, const XrFrameEndInfo* frameEndInfo)
{
	const XrResult Result = FunctionPassthroughs.EndFrame(session, frameEndInfo);
	CaptureEncoder.EncodeEndFrame(Result, session, frameEndInfo);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerLocateViews(XrSession session, const XrViewLocateInfo* viewLocateInfo, XrViewState* viewState, uint32_t viewCapacityInput, uint32_t* viewCountOutput, XrView* views)
{
	const XrResult Result = FunctionPassthroughs.LocateViews(session, viewLocateInfo, viewState, viewCapacityInput, viewCountOutput, views);
	CaptureEncoder.EncodeLocateViews(Result, session, viewLocateInfo, viewState, viewCapacityInput, viewCountOutput, views);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerStringToPath(XrInstance instance, const char* pathString, XrPath* path)
{
	const XrResult Result = FunctionPassthroughs.StringToPath(instance, pathString, path);
	CaptureEncoder.EncodeStringToPath(Result, instance, pathString, path);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerPathToString(XrInstance instance, XrPath path, uint32_t bufferCapacityInput, uint32_t* bufferCountOutput, char* buffer)
{
	const XrResult Result = FunctionPassthroughs.PathToString(instance, path, bufferCapacityInput, bufferCountOutput, buffer);
	CaptureEncoder.EncodePathToString(Result, instance, path, bufferCapacityInput, bufferCountOutput, buffer);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerCreateActionSet(XrInstance instance, const XrActionSetCreateInfo* createInfo, XrActionSet* actionSet)
{
	const XrResult Result = FunctionPassthroughs.CreateActionSet(instance, createInfo, actionSet);
	CaptureEncoder.EncodeCreateActionSet(Result, instance, createInfo, actionSet);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerDestroyActionSet(XrActionSet actionSet)
{
	const XrResult Result = FunctionPassthroughs.DestroyActionSet(actionSet);
	CaptureEncoder.EncodeDestroyActionSet(Result, actionSet);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerCreateAction(XrActionSet actionSet, const XrActionCreateInfo* createInfo, XrAction* action)
{
	const XrResult Result = FunctionPassthroughs.CreateAction(actionSet, createInfo, action);
	CaptureEncoder.EncodeCreateAction(Result, actionSet, createInfo, action);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerDestroyAction(XrAction action)
{
	const XrResult Result = FunctionPassthroughs.DestroyAction(action);
	CaptureEncoder.EncodeDestroyAction(Result, action);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerSuggestInteractionProfileBindings(XrInstance instance, const XrInteractionProfileSuggestedBinding* suggestedBindings)
{
	const XrResult Result = FunctionPassthroughs.SuggestInteractionProfileBindings(instance, suggestedBindings);
	CaptureEncoder.EncodeSuggestInteractionProfileBindings(Result, instance, suggestedBindings);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerAttachSessionActionSets(XrSession session, const XrSessionActionSetsAttachInfo* attachInfo)
{
	const XrResult Result = FunctionPassthroughs.AttachSessionActionSets(session, attachInfo);
	CaptureEncoder.EncodeAttachSessionActionSets(Result, session, attachInfo);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerGetCurrentInteractionProfile(XrSession session, XrPath topLevelUserPath, XrInteractionProfileState* interactionProfile)
{
	const XrResult Result = FunctionPassthroughs.GetCurrentInteractionProfile(session, topLevelUserPath, interactionProfile);
	CaptureEncoder.EncodeGetCurrentInteractionProfile(Result, session, topLevelUserPath, interactionProfile);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerGetActionStateBoolean(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStateBoolean* state)
{
	const XrResult Result = FunctionPassthroughs.GetActionStateBoolean(session, getInfo, state);
	CaptureEncoder.EncodeGetActionStateBoolean(Result, session, getInfo, state);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerGetActionStateFloat(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStateFloat* state)
{
	const XrResult Result = FunctionPassthroughs.GetActionStateFloat(session, getInfo, state);
	CaptureEncoder.EncodeGetActionStateFloat(Result, session, getInfo, state);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerGetActionStateVector2f(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStateVector2f* state)
{
	const XrResult Result = FunctionPassthroughs.GetActionStateVector2f(session, getInfo, state);
	CaptureEncoder.EncodeGetActionStateVector2f(Result, session, getInfo, state);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerGetActionStatePose(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStatePose* state)
{
	const XrResult Result = FunctionPassthroughs.GetActionStatePose(session, getInfo, state);
	CaptureEncoder.EncodeGetActionStatePose(Result, session, getInfo, state);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerSyncActions(XrSession session, const	XrActionsSyncInfo* syncInfo)
{
	const XrResult Result = FunctionPassthroughs.SyncActions(session, syncInfo);
	CaptureEncoder.EncodeSyncActions(Result, session, syncInfo);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerEnumerateBoundSourcesForAction(XrSession session, const XrBoundSourcesForActionEnumerateInfo* enumerateInfo, uint32_t sourceCapacityInput, uint32_t* sourceCountOutput, XrPath* sources)
{
	const XrResult Result = FunctionPassthroughs.EnumerateBoundSourcesForAction(session, enumerateInfo, sourceCapacityInput, sourceCountOutput, sources);
	CaptureEncoder.EncodeEnumerateBoundSourcesForAction(Result, session, enumerateInfo, sourceCapacityInput, sourceCountOutput, sources);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerGetInputSourceLocalizedName(XrSession session, const XrInputSourceLocalizedNameGetInfo* getInfo, uint32_t bufferCapacityInput, uint32_t* bufferCountOutput, char* buffer)
{
	const XrResult Result = FunctionPassthroughs.GetInputSourceLocalizedName(session, getInfo, bufferCapacityInput, bufferCountOutput, buffer);
	CaptureEncoder.EncodeGetInputSourceLocalizedName(Result, session, getInfo, bufferCapacityInput, bufferCountOutput, buffer);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerApplyHapticFeedback(XrSession session, const XrHapticActionInfo* hapticActionInfo, const XrHapticBaseHeader* hapticFeedback)
{
	const XrResult Result = FunctionPassthroughs.ApplyHapticFeedback(session, hapticActionInfo, hapticFeedback);
	CaptureEncoder.EncodeApplyHapticFeedback(Result, session, hapticActionInfo, hapticFeedback);
	return Result;
}
XrResult FOpenXRCaptureLayer::XrLayerStopHapticFeedback(XrSession session, const XrHapticActionInfo* hapticActionInfo)
{
	const XrResult Result = FunctionPassthroughs.StopHapticFeedback(session, hapticActionInfo);
	CaptureEncoder.EncodeStopHapticFeedback(Result, session, hapticActionInfo);
	return Result;
}

// Global extensions

// XR_KHR_loader_init
XrResult FOpenXRCaptureLayer::XrLayerInitializeLoaderKHR(const XrLoaderInitInfoBaseHeaderKHR* loaderInitInfo)
{ 
	// TODO: We won't actually capture this because of it's used in FOpenXRHMDModule::GetDefaultLoader
	// We need to move that code out of GetDefaultLoader and have it execute later in InitInstance.
	return XrResult::XR_ERROR_FUNCTION_UNSUPPORTED;
}

// Instance extensions

// XR_KHR_visibility_mask
XrResult FOpenXRCaptureLayer::XrLayerGetVisibilityMaskKHR(XrSession session, XrViewConfigurationType viewConfigurationType, uint32_t viewIndex, XrVisibilityMaskTypeKHR visibilityMaskType, XrVisibilityMaskKHR* visibilityMask)
{
	const XrResult Result = FunctionPassthroughs.GetVisibilityMaskKHR(session, viewConfigurationType, viewIndex, visibilityMaskType, visibilityMask);
	CaptureEncoder.EncodeGetVisibilityMaskKHR(Result, session, viewConfigurationType, viewIndex, visibilityMaskType, visibilityMask);
	return Result;
}

#if defined(XR_USE_GRAPHICS_API_D3D11)
// XR_KHR_D3D11_enable
XrResult FOpenXRCaptureLayer::XrLayerGetD3D11GraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsD3D11KHR* graphicsRequirements)
{
	const XrResult Result = FunctionPassthroughs.GetD3D11GraphicsRequirementsKHR(instance, systemId, graphicsRequirements);
	CaptureEncoder.EncodeGetD3D11GraphicsRequirementsKHR(Result, instance, systemId, graphicsRequirements);
	return Result;
}
#endif

#if defined(XR_USE_GRAPHICS_API_D3D12)
// XR_KHR_D3D12_enable
XrResult FOpenXRCaptureLayer::XrLayerGetD3D12GraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsD3D12KHR* graphicsRequirements)
{
	const XrResult Result = FunctionPassthroughs.GetD3D12GraphicsRequirementsKHR(instance, systemId, graphicsRequirements);
	CaptureEncoder.EncodeGetD3D12GraphicsRequirementsKHR(Result, instance, systemId, graphicsRequirements);
	return Result;
}
#endif

#if defined(XR_USE_GRAPHICS_API_OPENGL)
// XR_KHR_opengl_enable
XrResult FOpenXRCaptureLayer::XrLayerGetOpenGLGraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsOpenGLKHR* graphicsRequirements)
{
	return XrResult::XR_ERROR_FUNCTION_UNSUPPORTED;
}
#endif

#if defined(XR_USE_GRAPHICS_API_VULKAN)
// XR_KHR_vulkan_enable
XrResult FOpenXRCaptureLayer::XrLayerGetVulkanGraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsVulkanKHR* graphicsRequirements) 
{
	return XrResult::XR_ERROR_FUNCTION_UNSUPPORTED; 
}
#endif

} // namespace UE::XRScribe
