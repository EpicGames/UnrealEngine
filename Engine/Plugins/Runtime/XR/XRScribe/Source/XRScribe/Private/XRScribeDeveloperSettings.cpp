// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRScribeDeveloperSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(XRScribeDeveloperSettings)

UXRScribeDeveloperSettings::UXRScribeDeveloperSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
}

FName UXRScribeDeveloperSettings::GetCategoryName() const
{
	return FName(TEXT("Plugins"));
}