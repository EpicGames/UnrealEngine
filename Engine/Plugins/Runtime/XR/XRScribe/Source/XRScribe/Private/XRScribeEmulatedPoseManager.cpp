// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRScribeEmulatedPoseManager.h"
#include "XRScribeFileFormat.h"

#include "HAL/IConsoleManager.h"
#include "Logging/LogMacros.h"

DEFINE_LOG_CATEGORY_STATIC(LogXRScribePoseManager, Log, All);

namespace UE::XRScribe
{

enum class EXRScribePoseReplayMode : int32
{
	/** Replay poses over the same time period they were captured */
	TimeMatched,
	/** Replay poses as fast as possible (might look slow or fast across different machines/build configs) */
	Immediate,
};

 static EXRScribePoseReplayMode XRScribePoseReplayMode = EXRScribePoseReplayMode::TimeMatched;
 static FAutoConsoleVariableRef CVarXRScribePoseReplay(TEXT("XRScribe.PoseReplayMode"),
 	reinterpret_cast<int32&>(XRScribePoseReplayMode),
 	TEXT("Toggle the pose replay mode for XRScribe (TimeMatched, Immediate)"),
 	ECVF_ReadOnly);

 static int32 ConstrainActionTime = 5;
 static FAutoConsoleVariableRef CVarXRScribeConstrainActionTime(TEXT("XRScribe.ConstrainActionTime"),
	 ConstrainActionTime,
	 TEXT("Limit the earliest action time to be within N frames of the first predictedDisplayTime, -1 disables"),
	 ECVF_ReadOnly);
	
 void FOpenXRActionPoseManager::RegisterCapturedPathStrings(const TMap<XrPath, FName>& PathStringMap)
 {
	 CapturedPaths = PathStringMap;
 }

void FOpenXRActionPoseManager::RegisterCapturedWaitFrames(const TArray<FOpenXRWaitFramePacket>& InWaitFrameHistory)
{
	WaitFrameHistory = InWaitFrameHistory;
}

void FOpenXRActionPoseManager::RegisterCapturedReferenceSpaces(const TArray<FOpenXRCreateReferenceSpacePacket>& CreateReferenceSpacePackets)
{
	CapturedReferenceSpaces.Append(CreateReferenceSpacePackets);

	// TODO: check for doubles?
}

XrReferenceSpaceType FOpenXRActionPoseManager::GetCapturedReferenceSpaceType(XrSpace CapturedSpace)
{
	XrReferenceSpaceType Result = XR_REFERENCE_SPACE_TYPE_MAX_ENUM;

	FOpenXRCreateReferenceSpacePacket* MatchingInfo = CapturedReferenceSpaces.FindByPredicate([CapturedSpace](const FOpenXRCreateReferenceSpacePacket& CreateRefSpaceInfo)
	{
		return CreateRefSpaceInfo.Space == CapturedSpace;
	});

	if (MatchingInfo != nullptr)
	{
		Result = MatchingInfo->ReferenceSpaceCreateInfo.referenceSpaceType;
	}

	return Result;
}

void FOpenXRActionPoseManager::RegisterCapturedActions(const TArray <FOpenXRCreateActionPacket>& CreateActionPackets)
{
	for (const FOpenXRCreateActionPacket& CreateAction : CreateActionPackets)
	{
		CapturedActions.Add(CreateAction.Action, CreateAction);
	}
}

void FOpenXRActionPoseManager::RegisterCapturedActionSpaces(const TArray<FOpenXRCreateActionSpacePacket>& CreateActionSpacePackets)
{
	CapturedActionSpaces.Append(CreateActionSpacePackets);
}

bool FOpenXRActionPoseManager::VerifyCapturedActionSpace(XrSpace CapturedSpace)
{
	return CapturedActionSpaces.ContainsByPredicate([CapturedSpace](const FOpenXRCreateActionSpacePacket& CreateActionSpaceInfo)
	{
		return CreateActionSpaceInfo.Space == CapturedSpace;
	});
}

bool FOpenXRActionPoseManager::FilterSpaceHistory(const TArray<FOpenXRLocateSpacePacket>& RawHistory, TArray<FOpenXRLocateSpacePacket>& FilteredHistory)
{
	XrSpace CapturedBaseSpace = RawHistory[0].BaseSpace;
	XrReferenceSpaceType BaseRefSpaceType = GetCapturedReferenceSpaceType(CapturedBaseSpace);

	if (BaseRefSpaceType != XR_REFERENCE_SPACE_TYPE_STAGE)
	{
		UE_LOGF(LogXRScribePoseManager, Warning, "Unable to process space history with non-stage base space");
		return false;
	}

	// we now have our core case: locate space relative to stage tracker
	// Pick the last one with matching frametime, as the 'latest' recorded query should have most accurate location.
	// We might want to convert to UE FTransform later on to facilitate easy transformations.
	// TODO: account for different starting poses in stage space

	FilteredHistory = RawHistory;
	FilteredHistory.Sort([](const FOpenXRLocateSpacePacket& A, const FOpenXRLocateSpacePacket& B)
	{
		return A.Time < B.Time;
	});

	// We could also do this in place like Algo::Unique, putting elements in the front of the sorted array
	// But we want to select the 'last' element in the run sequence
	int32 InsertIndex = 0;
	for (int32 LocationIndex = 0; LocationIndex < FilteredHistory.Num() - 1; LocationIndex++)
	{
		if (FilteredHistory[LocationIndex].Time != FilteredHistory[LocationIndex + 1].Time)
		{
			FilteredHistory[InsertIndex++] = FilteredHistory[LocationIndex];
		}
	}
	FilteredHistory[InsertIndex++] = FilteredHistory.Last();

	FilteredHistory.SetNumUninitialized(InsertIndex);

	// TODO: clients should transform these sorted poses into a unified space independent of changing base space

	return true;
}

TArray<FOpenXRLocateSpacePacket> FOpenXRActionPoseManager::SliceFilteredSpaceHistory(const TArray<FOpenXRLocateSpacePacket>& FilteredHistory)
{
	TArray<FOpenXRLocateSpacePacket> SlicedHistory;

	if (XRScribePoseReplayMode == EXRScribePoseReplayMode::TimeMatched)
	{
		SlicedHistory.Reserve(CapturedSliceCount);

		FOpenXRLocateSpacePacket PropogatedLocationPacket = FilteredHistory.Top();
		PropogatedLocationPacket.Location.locationFlags = XR_SPACE_LOCATION_ORIENTATION_VALID_BIT | XR_SPACE_LOCATION_POSITION_VALID_BIT | XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT | XR_SPACE_LOCATION_POSITION_TRACKED_BIT;
		PropogatedLocationPacket.Location.pose.orientation = ToXrQuat(FQuat::Identity);
		PropogatedLocationPacket.Location.pose.position = ToXrVector(FVector::ZeroVector);

		int64 HistoryIndex = 0;
		for (int64 SliceIndex = 0; SliceIndex < CapturedSliceCount; SliceIndex++)
		{
			PropogatedLocationPacket.Time = CapturedRangeStart + (SliceIndex * 8333333);

			// TODO: we could interpolate poses in the slice?
			// TODO: should we clean out invalid poses?

			// forward propagate current pose until slice time surpasses current history time
			if (HistoryIndex < FilteredHistory.Num() &&
				FilteredHistory[HistoryIndex].Time <= PropogatedLocationPacket.Time)
			{
				PropogatedLocationPacket.Location = FilteredHistory[HistoryIndex].Location;
				HistoryIndex++;
			}
		
			SlicedHistory.Add(PropogatedLocationPacket);
		}
	}
	else
	{
		check(XRScribePoseReplayMode == EXRScribePoseReplayMode::Immediate);
		SlicedHistory = FilteredHistory;
	}

	return SlicedHistory;
}

void FOpenXRActionPoseManager::ProcessCapturedReferenceSpaceHistory(const TArray<FOpenXRLocateSpacePacket>& SpaceHistory)
{
	const XrSpace CapturedLocatedSpace = SpaceHistory[0].Space;
	const XrReferenceSpaceType RefSpaceType = GetCapturedReferenceSpaceType(CapturedLocatedSpace);

	if (RefSpaceType != XR_REFERENCE_SPACE_TYPE_VIEW)
	{
		UE_LOGF(LogXRScribePoseManager, Warning, "Unable to process space history for non-view reference spaces");
		return;
	}

	TArray<FOpenXRLocateSpacePacket> FilteredSpaceHistory;
	if (FilterSpaceHistory(SpaceHistory, FilteredSpaceHistory))
	{
		TArray<FOpenXRLocateSpacePacket> SlicedHistory = SliceFilteredSpaceHistory(FilteredSpaceHistory);

		ReferencePoseHistories.Add(XR_REFERENCE_SPACE_TYPE_VIEW, MoveTemp(SlicedHistory));
	}
}

void FOpenXRActionPoseManager::ProcessCapturedActionSpaceHistory(const TArray<FOpenXRLocateSpacePacket>& SpaceHistory)
{
	const XrSpace LocatedSpace = SpaceHistory[0].Space;

	FOpenXRCreateActionSpacePacket* MatchingInfo = CapturedActionSpaces.FindByPredicate([LocatedSpace](const FOpenXRCreateActionSpacePacket& CreateActionSpaceInfo)
	{
		return CreateActionSpaceInfo.Space == LocatedSpace;

	});
	check(MatchingInfo);

	const FOpenXRCreateActionPacket& CreateAction = CapturedActions[MatchingInfo->ActionSpaceCreateInfo.action];

	check(CreateAction.ActionType == XR_ACTION_TYPE_POSE_INPUT);

	TArray<FOpenXRLocateSpacePacket> FilteredSpaceHistory;
	if (FilterSpaceHistory(SpaceHistory, FilteredSpaceHistory))
	{
		const FName FetchedActionName(CreateAction.ActionName.GetData());

		TArray<FOpenXRLocateSpacePacket> SlicedHistory = SliceFilteredSpaceHistory(FilteredSpaceHistory);
		ActionPoseHistories.Add(FetchedActionName, MoveTemp(SlicedHistory));

		// TODO: we should be resilient to multiple action spaces per name
	}
}

void FOpenXRActionPoseManager::RegisterCapturedSpaceHistories(const TMap<XrSpace, TArray<FOpenXRLocateSpacePacket>>& SpaceHistories)
{
	CapturedRawSpaceHistories = SpaceHistories;
}

void FOpenXRActionPoseManager::RegisterCapturedActionStates(const TArray<FOpenXRSyncActionsPacket>& SyncActionsPackets,
	const TMap<XrAction, TArray<FOpenXRGetActionStateBooleanPacket>>& BooleanActionStates,
	const TMap<XrAction, TArray<FOpenXRGetActionStateFloatPacket>>& FloatActionStates,
	const TMap<XrAction, TArray<FOpenXRGetActionStateVector2fPacket>>& VectorActionStates,
	const TMap<XrAction, TArray<FOpenXRGetActionStatePosePacket>>& PoseActionStates)
{

	//CapturedRawSyncActionsPackets = SyncActionsPackets;
	CapturedRawBooleanActionStates = BooleanActionStates;
	CapturedRawFloatActionStates = FloatActionStates;
	CapturedRawVectorActionStates = VectorActionStates;
	CapturedRawPoseActionStates = PoseActionStates;
}

void FOpenXRActionPoseManager::CalculateCapturedTimeRange()
{
	// TODO: We might want to offset this time 'back' based on the predictedDisplayPeriod. That would estimate the CPU time start for the session
	// TODO: We might want to calculate a 'real' last time based on the latest time returned from an ActionState lastChangeTime
	XrTime WaitFrameStart = WaitFrameHistory[0].FrameState.predictedDisplayTime;
	XrTime WaitFrameEnd = WaitFrameHistory.Last().FrameState.predictedDisplayTime;

	XrTime EarliestActionTime = INT64_MAX;
	XrTime LatestActionTime = 0;

	for (const auto& ActionStateHistory : CapturedRawBooleanActionStates)
	{
		uint32 NumActive = 0;
		for (const FOpenXRGetActionStateBooleanPacket& BooleanActionState : ActionStateHistory.Value)
		{
			if (BooleanActionState.BooleanState.isActive && BooleanActionState.BooleanState.changedSinceLastSync)
			{
				EarliestActionTime = FMath::Min(EarliestActionTime, BooleanActionState.BooleanState.lastChangeTime);
				LatestActionTime = FMath::Max(LatestActionTime, BooleanActionState.BooleanState.lastChangeTime);
				NumActive++;
			}
		}
		UE_LOGF(LogXRScribePoseManager, Verbose, "Bool action %p - active count: %d / %d", ActionStateHistory.Key, NumActive, ActionStateHistory.Value.Num());
	}
	for (const auto& ActionStateHistory : CapturedRawFloatActionStates)
	{
		uint32 NumActive = 0;
		for (const FOpenXRGetActionStateFloatPacket& FloatActionState : ActionStateHistory.Value)
		{
			if (FloatActionState.FloatState.isActive && FloatActionState.FloatState.changedSinceLastSync)
			{
				EarliestActionTime = FMath::Min(EarliestActionTime, FloatActionState.FloatState.lastChangeTime);
				LatestActionTime = FMath::Max(LatestActionTime, FloatActionState.FloatState.lastChangeTime);
				NumActive++;
			}
		}
		UE_LOGF(LogXRScribePoseManager, Verbose, "Float action %p - active count: %d / %d", ActionStateHistory.Key, NumActive, ActionStateHistory.Value.Num());
	}
	for (const auto& ActionStateHistory : CapturedRawVectorActionStates)
	{
		uint32 NumActive = 0;
		for (const FOpenXRGetActionStateVector2fPacket& VectorActionState : ActionStateHistory.Value)
		{
			if (VectorActionState.Vector2fState.isActive && VectorActionState.Vector2fState.changedSinceLastSync)
			{
				EarliestActionTime = FMath::Min(EarliestActionTime, VectorActionState.Vector2fState.lastChangeTime);
				LatestActionTime = FMath::Max(LatestActionTime, VectorActionState.Vector2fState.lastChangeTime);
				NumActive++;
			}
		}
		UE_LOGF(LogXRScribePoseManager, Verbose, "Vector2F action %p - active count: %d / %d", ActionStateHistory.Key, NumActive, ActionStateHistory.Value.Num());
	}

	CapturedRangeStart = FMath::Min(EarliestActionTime, WaitFrameStart);
	CapturedRangeEnd = FMath::Max(LatestActionTime, WaitFrameEnd);

	const XrTime ConstrainedEarliestTime = (ConstrainActionTime >= 0) ?
		(WaitFrameStart - (WaitFrameHistory[0].FrameState.predictedDisplayPeriod * ConstrainActionTime)) :
		0;
	CapturedRangeStart = FMath::Max(CapturedRangeStart, ConstrainedEarliestTime);
}

void FOpenXRActionPoseManager::ProcessCapturedHistories()
{
	CalculateCapturedTimeRange();

	// generate 120Hz slices for entire capture, as we assume we won't sample poses faster than 120Hz
	// TODO: 240 Hz might work better across disparate time domains (WaitFrame, SyncActions, GetActionState, LocateViews, LocateSpace)
	CapturedSliceCount = ((CapturedRangeEnd - CapturedRangeStart) / 8333333) + 1;

	for (const auto& SpaceHistoryTuple : CapturedRawSpaceHistories)
	{
		const TArray<FOpenXRLocateSpacePacket>& SpaceHistory = SpaceHistoryTuple.Value;
		if (SpaceHistory.Num() == 0)
		{
			continue;
		}

		const XrSpace CapturedLocatedSpace = SpaceHistoryTuple.Key;
		check(CapturedLocatedSpace == SpaceHistory[0].Space);

		const XrReferenceSpaceType RefSpaceType = GetCapturedReferenceSpaceType(CapturedLocatedSpace);
		if (RefSpaceType != XR_REFERENCE_SPACE_TYPE_MAX_ENUM)
		{
			ProcessCapturedReferenceSpaceHistory(SpaceHistory);
		}
		else if (VerifyCapturedActionSpace(CapturedLocatedSpace))
		{
			ProcessCapturedActionSpaceHistory(SpaceHistory);
		}
		else
		{
			UE_LOGF(LogXRScribePoseManager, Warning, "Unable to process space history for captured space: %p", reinterpret_cast<const void*>(CapturedLocatedSpace));
		}
	}

	// Converting raw action state captures into a processed state history
	// * Extract valid subpaths from captured state
	// * Filter out 'invalid' state values
	//   * invalid = inactive, changedSinceLastSync is false, or lastChangeTime is 'unreasonably' early 
	//   * We might want to support intermediate inactive periods in the future
	//	   Currently, we do not support 'holes' in the state history.
	// * Compress list of active states to reflect changes only (based on change time)
	// * Based on initial 'active' time, prepend history with inactive states to match up with full capture timespan
	// * Slice state changes into 120 Hz history of states
	// * add in last known state to complete unknown history

	for (const auto& BooleanActionStateList : CapturedRawBooleanActionStates)
	{
		FBooleanActionStateHistory ProcessedBooleanStateHistory;
		ProcessedBooleanStateHistory.StateValues.Reserve(CapturedSliceCount);

		// Filter list into 'Active' entries
		// TODO: This might not be entirely accurate. We won't capture 'holes' in the history
		// which could happen if an action is out of scope for any reason. But for our initial
		// implementation, it's fine.

		TArray<FOpenXRGetActionStateBooleanPacket> ActiveBooleanStateList;
		ActiveBooleanStateList.Reserve(BooleanActionStateList.Value.Num());

		// TODO: Scan for any valid subpaths
		for (const FOpenXRGetActionStateBooleanPacket& Packet : BooleanActionStateList.Value)
		{
			if (Packet.BooleanState.isActive == XR_TRUE && 
				Packet.BooleanState.changedSinceLastSync &&
				Packet.BooleanState.lastChangeTime >= CapturedRangeStart)
			{
				ActiveBooleanStateList.Add(Packet);
				if (Packet.GetInfoBoolean.subactionPath != XR_NULL_PATH)
				{
					ProcessedBooleanStateHistory.ValidSubpaths.Add(CapturedPaths[Packet.GetInfoBoolean.subactionPath]);
				}
			}
		}

		if (ActiveBooleanStateList.IsEmpty())
		{
			continue;
		}

		// Compress the active list to remove redundant entries
		TArray<FOpenXRGetActionStateBooleanPacket> CompressedActiveBooleanStateList;
		CompressedActiveBooleanStateList.Reserve(ActiveBooleanStateList.Num());
		CompressedActiveBooleanStateList.Add(ActiveBooleanStateList[0]);
		for (const FOpenXRGetActionStateBooleanPacket& Packet : ActiveBooleanStateList)
		{
			if (Packet.BooleanState.lastChangeTime != CompressedActiveBooleanStateList.Last().BooleanState.lastChangeTime)
			{
				CompressedActiveBooleanStateList.Add(Packet);
			}
		}

		// TODO: Prefill any slices with 'inactive'
		int64 SliceIndex = 0;

		while (SliceIndex < CapturedSliceCount)
		{
			const XrTime SliceTime = CapturedRangeStart + (SliceIndex * 8333333);

			if (SliceTime >= CompressedActiveBooleanStateList[0].BooleanState.lastChangeTime)
			{
				break;
			}

			ProcessedBooleanStateHistory.StateValues.Add(FBooleanActionState());

			SliceIndex++;
		}

		// Double walk lists to insert state changes
		int64 HistoryIndex = 0;

		while ((SliceIndex < CapturedSliceCount) &&
			   (HistoryIndex < CompressedActiveBooleanStateList.Num()))
		{
			const XrTime SliceTime = CapturedRangeStart + (SliceIndex * 8333333);

			if (SliceTime >= CompressedActiveBooleanStateList[HistoryIndex].BooleanState.lastChangeTime)
			{
				FBooleanActionState StateEntry;
				StateEntry.Active = XR_TRUE;
				StateEntry.State = CompressedActiveBooleanStateList[HistoryIndex].BooleanState.currentState;
				ProcessedBooleanStateHistory.StateValues.Add(StateEntry);

				HistoryIndex++;
			}
			else
			{
				// Forward propogate state from last slice
				// TODO: is it possible to ever forward propogate for index 0??
				FBooleanActionState PrevState = ProcessedBooleanStateHistory.StateValues.Last();
				ProcessedBooleanStateHistory.StateValues.Add(PrevState);
			}

			SliceIndex++;
		}

		// TODO: Fill in remainder with last known state
		while (SliceIndex < CapturedSliceCount)
		{
			FBooleanActionState PrevState = ProcessedBooleanStateHistory.StateValues.Last();
			ProcessedBooleanStateHistory.StateValues.Add(PrevState);
			SliceIndex++;
		}

		// TODO: support arbitrary internal ranges of inactive
		// Will need to match SyncAction calls to WaitFrames in order to estimate inactive
		// time ranges

		// add processed list to the map with the name
		const FOpenXRCreateActionPacket& CreateActionPacket = CapturedActions[BooleanActionStateList.Key];
		const FName ActionName(CreateActionPacket.ActionName.GetData());
		BooleanProcessedHistories.Add(ActionName, ProcessedBooleanStateHistory);
	}	

	for (const auto& FloatActionStateList : CapturedRawFloatActionStates)
	{
		FFloatActionStateHistory ProcessedFloatStateHistory;
		ProcessedFloatStateHistory.StateValues.Reserve(CapturedSliceCount);

		TArray<FOpenXRGetActionStateFloatPacket> ActiveFloatStateList;
		ActiveFloatStateList.Reserve(FloatActionStateList.Value.Num());

		for (const FOpenXRGetActionStateFloatPacket& Packet : FloatActionStateList.Value)
		{
			if (Packet.FloatState.isActive == XR_TRUE && 
				Packet.FloatState.changedSinceLastSync &&
				Packet.FloatState.lastChangeTime >= CapturedRangeStart)
			{
				ActiveFloatStateList.Add(Packet);
				if (Packet.GetInfoFloat.subactionPath != XR_NULL_PATH)
				{
					ProcessedFloatStateHistory.ValidSubpaths.Add(CapturedPaths[Packet.GetInfoFloat.subactionPath]);
				}
			}
		}

		if (ActiveFloatStateList.IsEmpty())
		{
			continue;
		}

		TArray<FOpenXRGetActionStateFloatPacket> CompressedActiveFloatStateList;
		CompressedActiveFloatStateList.Reserve(ActiveFloatStateList.Num());
		CompressedActiveFloatStateList.Add(ActiveFloatStateList[0]);
		for (const FOpenXRGetActionStateFloatPacket& Packet : ActiveFloatStateList)
		{
			if (Packet.FloatState.lastChangeTime != CompressedActiveFloatStateList.Last().FloatState.lastChangeTime)
			{
				CompressedActiveFloatStateList.Add(Packet);
			}
		}

		int64 SliceIndex = 0;

		while (SliceIndex < CapturedSliceCount)
		{
			const XrTime SliceTime = CapturedRangeStart + (SliceIndex * 8333333);
			if (SliceTime >= CompressedActiveFloatStateList[0].FloatState.lastChangeTime)
			{
				break;
			}
			ProcessedFloatStateHistory.StateValues.Add(FFloatActionState());
			SliceIndex++;
		}

		int64 HistoryIndex = 0;

		while ((SliceIndex < CapturedSliceCount) &&
			(HistoryIndex < CompressedActiveFloatStateList.Num()))
		{
			FFloatActionState StateEntry;
			const XrTime SliceTime = CapturedRangeStart + (SliceIndex * 8333333);
			if (SliceTime >= CompressedActiveFloatStateList[HistoryIndex].FloatState.lastChangeTime)
			{
				StateEntry.Active = XR_TRUE;
				StateEntry.State = CompressedActiveFloatStateList[HistoryIndex].FloatState.currentState;
				HistoryIndex++;
			}
			else
			{
				// Forward propogate state from last slice
				// TODO: is it possible to ever forward propogate for index 0??
				StateEntry = ProcessedFloatStateHistory.StateValues.Last();
			}
			ProcessedFloatStateHistory.StateValues.Add(StateEntry);
			SliceIndex++;
		}

		while (SliceIndex < CapturedSliceCount)
		{
			FFloatActionState PrevState = ProcessedFloatStateHistory.StateValues.Last();
			ProcessedFloatStateHistory.StateValues.Add(PrevState);
			SliceIndex++;
		}

		// TODO: support arbitrary internal ranges of inactive
		// Will need to match SyncAction calls to WaitFrames in order to estimate inactive
		// time ranges

		const FOpenXRCreateActionPacket& CreateActionPacket = CapturedActions[FloatActionStateList.Key];
		const FName ActionName(CreateActionPacket.ActionName.GetData());
		FloatProcessedHistories.Add(ActionName, ProcessedFloatStateHistory);
	}

	for (const auto& VectorActionStateList : CapturedRawVectorActionStates)
	{
		FVector2fActionStateHistory ProcessedVector2fStateHistory;
		ProcessedVector2fStateHistory.StateValues.Reserve(CapturedSliceCount);

		TArray<FOpenXRGetActionStateVector2fPacket> ActiveVector2fStateList;
		ActiveVector2fStateList.Reserve(VectorActionStateList.Value.Num());

		for (const FOpenXRGetActionStateVector2fPacket& Packet : VectorActionStateList.Value)
		{
			if (Packet.Vector2fState.isActive == XR_TRUE && 
				Packet.Vector2fState.changedSinceLastSync &&
				Packet.Vector2fState.lastChangeTime >= CapturedRangeStart)
			{
				ActiveVector2fStateList.Add(Packet);
				if (Packet.GetInfoVector2f.subactionPath != XR_NULL_PATH)
				{
					ProcessedVector2fStateHistory.ValidSubpaths.Add(CapturedPaths[Packet.GetInfoVector2f.subactionPath]);
				}
			}
		}

		if (ActiveVector2fStateList.IsEmpty())
		{
			continue;
		}

		TArray<FOpenXRGetActionStateVector2fPacket> CompressedActiveVector2fStateList;
		CompressedActiveVector2fStateList.Reserve(ActiveVector2fStateList.Num());
		CompressedActiveVector2fStateList.Add(ActiveVector2fStateList[0]);
		for (const FOpenXRGetActionStateVector2fPacket& Packet : ActiveVector2fStateList)
		{
			if (Packet.Vector2fState.lastChangeTime != CompressedActiveVector2fStateList.Last().Vector2fState.lastChangeTime)
			{
				CompressedActiveVector2fStateList.Add(Packet);
			}
		}

		int64 SliceIndex = 0;

		while (SliceIndex < CapturedSliceCount)
		{
			const XrTime SliceTime = CapturedRangeStart + (SliceIndex * 8333333);
			if (SliceTime >= CompressedActiveVector2fStateList[0].Vector2fState.lastChangeTime)
			{
				break;
			}
			ProcessedVector2fStateHistory.StateValues.Add(FVector2fActionState());
			SliceIndex++;
		}

		int64 HistoryIndex = 0;

		while ((SliceIndex < CapturedSliceCount) &&
			(HistoryIndex < CompressedActiveVector2fStateList.Num()))
		{
			FVector2fActionState StateEntry;
			const XrTime SliceTime = CapturedRangeStart + (SliceIndex * 8333333);
			if (SliceTime >= CompressedActiveVector2fStateList[HistoryIndex].Vector2fState.lastChangeTime)
			{
				StateEntry.Active = XR_TRUE;
				StateEntry.State = CompressedActiveVector2fStateList[HistoryIndex].Vector2fState.currentState;
				HistoryIndex++;
			}
			else
			{
				// Forward propogate state from last slice
				// TODO: is it possible to ever forward propogate for index 0??
				StateEntry = ProcessedVector2fStateHistory.StateValues.Last();
			}
			ProcessedVector2fStateHistory.StateValues.Add(StateEntry);
			SliceIndex++;
		}

		while (SliceIndex < CapturedSliceCount)
		{
			FVector2fActionState PrevState = ProcessedVector2fStateHistory.StateValues.Last();
			ProcessedVector2fStateHistory.StateValues.Add(PrevState);
			SliceIndex++;
		}

		// TODO: support arbitrary internal ranges of inactive
		// Will need to match SyncAction calls to WaitFrames in order to estimate inactive
		// time ranges

		const FOpenXRCreateActionPacket& CreateActionPacket = CapturedActions[VectorActionStateList.Key];
		const FName ActionName(CreateActionPacket.ActionName.GetData());
		Vector2fProcessedHistories.Add(ActionName, ProcessedVector2fStateHistory);
	}
}

void FOpenXRActionPoseManager::RegisterEmulatedPath(FName PathString, XrPath Path)
{
	EmulatedPaths.Add(Path, PathString);
}

void FOpenXRActionPoseManager::RegisterEmulatedAction(FName ActionName, XrAction Action, XrActionType ActionType)
{
	EmulatedActionNames.Add(Action, ActionName);

	if (ActionType == XR_ACTION_TYPE_BOOLEAN_INPUT)
	{
		XrActionStateBoolean InitialEmulatedState{};
		InitialEmulatedState.type = XR_TYPE_ACTION_STATE_BOOLEAN;
		InitialEmulatedState.currentState = XR_FALSE;
		InitialEmulatedState.changedSinceLastSync = XR_FALSE;
		InitialEmulatedState.lastChangeTime = INT64_MAX;
		InitialEmulatedState.isActive = XR_FALSE;

		SetCachedEmulatedBooleanState(ActionName, InitialEmulatedState);
	}
	else if (ActionType == XR_ACTION_TYPE_FLOAT_INPUT)
	{
		XrActionStateFloat InitialEmulatedState{};
		InitialEmulatedState.type = XR_TYPE_ACTION_STATE_FLOAT;
		InitialEmulatedState.currentState = 0.0f;
		InitialEmulatedState.changedSinceLastSync = XR_FALSE;
		InitialEmulatedState.lastChangeTime = INT64_MAX;
		InitialEmulatedState.isActive = XR_FALSE;

		SetCachedEmulatedFloatState(ActionName, InitialEmulatedState);
	}
	else if (ActionType == XR_ACTION_TYPE_VECTOR2F_INPUT)
	{
		XrActionStateVector2f InitialEmulatedState{};
		InitialEmulatedState.type = XR_TYPE_ACTION_STATE_VECTOR2F;
		InitialEmulatedState.currentState = { 0.0f, 0.0f };
		InitialEmulatedState.changedSinceLastSync = XR_FALSE;
		InitialEmulatedState.lastChangeTime = INT64_MAX;
		InitialEmulatedState.isActive = XR_FALSE;

		SetCachedEmulatedVector2fState(ActionName, InitialEmulatedState);
	}
	else
	{
		// Pose states? What else?
	}
}

FName FOpenXRActionPoseManager::GetEmulatedActionName(XrAction Action)
{
	if (EmulatedActionNames.Contains(Action))
	{
		return EmulatedActionNames[Action];
	}
	return FName();
}

int64 FOpenXRActionPoseManager::GenerateReplaySliceIndexFromTime(XrTime EmulatedTime)
{
	const XrTime TimeOffsetFromSessionStart = EmulatedTime - EmulatedBaseTime;
	const int64 SliceOffset = TimeOffsetFromSessionStart / 8333333;

	// We run through the replay once, and retain the last pose/state.
	// We could also loop, but we'd need 'reset to origin' functionality to prevent unexpected loop interactions.
	return FMath::Min(SliceOffset, CapturedSliceCount - 1);
}

bool FOpenXRActionPoseManager::ValidateSubpath(const TSet<FName>& ValidSubpaths, XrPath ActionSubPath)
{
	bool bSubpathValid = true;
	if (ActionSubPath != XR_NULL_PATH)
	{
		if (EmulatedPaths.Contains(ActionSubPath))
		{
			const FName SubactionPathName = EmulatedPaths[ActionSubPath];
			bSubpathValid = ValidSubpaths.Contains(SubactionPathName);
		}
		else
		{
			bSubpathValid = false;
			UE_LOGF(LogXRScribePoseManager, Warning, "Unknown emulated path (%lld) to check against action subpaths", ActionSubPath);
		}
	}
	return bSubpathValid;
}

const XrActionStateBoolean FOpenXRActionPoseManager::GetCachedEmulatedBooleanState(FName ActionName)
{
	if (CachedEmulatedBooleanStates.Contains(ActionName))
	{
		return CachedEmulatedBooleanStates[ActionName];
	}
	// TODO: check/ensure?
	return XrActionStateBoolean({});
}

void FOpenXRActionPoseManager::SetCachedEmulatedBooleanState(FName ActionName, const XrActionStateBoolean& EmulatedState)
{
	CachedEmulatedBooleanStates.Add(ActionName, EmulatedState);
}

const XrActionStateFloat FOpenXRActionPoseManager::GetCachedEmulatedFloatState(FName ActionName)
{
	if (CachedEmulatedFloatStates.Contains(ActionName))
	{
		return CachedEmulatedFloatStates[ActionName];
	}
	// TODO: check/ensure?
	return XrActionStateFloat({});
}

void FOpenXRActionPoseManager::SetCachedEmulatedFloatState(FName ActionName, const XrActionStateFloat& EmulatedState)
{
	CachedEmulatedFloatStates.Add(ActionName, EmulatedState);
}

const XrActionStateVector2f FOpenXRActionPoseManager::GetCachedEmulatedVector2fState(FName ActionName)
{
	if (CachedEmulatedVector2fStates.Contains(ActionName))
	{
		return CachedEmulatedVector2fStates[ActionName];
	}
	// TODO: check/ensure?
	return XrActionStateVector2f({});
}

void FOpenXRActionPoseManager::SetCachedEmulatedVector2fState(FName ActionName, const XrActionStateVector2f& EmulatedState)
{
	CachedEmulatedVector2fStates.Add(ActionName, EmulatedState);
}


void FOpenXRActionPoseManager::RegisterEmulatedReferenceSpace(const XrReferenceSpaceCreateInfo& CreateInfo, XrSpace SpaceHandle)
{
	check(!EmulatedReferenceSpaceTypeMap.Contains(SpaceHandle));
	EmulatedReferenceSpaceTypeMap.Add(SpaceHandle, CreateInfo.referenceSpaceType);
}

void FOpenXRActionPoseManager::RegisterEmulatedActionSpace(TStaticArray<ANSICHAR, XR_MAX_ACTION_NAME_SIZE>& ActionName, const XrActionSpaceCreateInfo& SpaceCreateInfo, XrSpace SpaceHandle)
{
	check(!EmulatedActionSpaceNameMap.Contains(SpaceHandle));

	const FName EmulatedActionName(ActionName.GetData());
	EmulatedActionSpaceNameMap.Add(SpaceHandle, EmulatedActionName);
}

void FOpenXRActionPoseManager::AddEmulatedFrameTime(XrTime Time, int32 FrameNum)
{
	// TODO: protect against repeated frame times. would a runtime even do that? I guess it could happen,
	// but this is for our runtime, and we won't do that for now
	// TODO: protect against repeated frame numbers?

	const int32 HistoryIndex = FrameNum % PoseHistorySize;
	FrameTimeHistory[HistoryIndex] = FrameTimeHistoryEntry({ Time , FrameNum });

	LastInsertedFrameIndex = HistoryIndex;

	EmulatedBaseTime = FMath::Min(EmulatedBaseTime, Time);
}

bool FOpenXRActionPoseManager::DoesActionContainPoseHistory(TStaticArray<ANSICHAR, XR_MAX_ACTION_NAME_SIZE>& ActionName)
{
	return ActionPoseHistories.Contains(ActionName.GetData());
}

XrSpaceLocation FOpenXRActionPoseManager::GetEmulatedPoseForTime(XrSpace LocatingSpace, XrSpace BaseSpace, XrTime Time)
{
	XrSpaceLocation DummyLocation;
	DummyLocation.locationFlags = XR_SPACE_LOCATION_ORIENTATION_VALID_BIT | XR_SPACE_LOCATION_POSITION_VALID_BIT | XR_SPACE_LOCATION_ORIENTATION_TRACKED_BIT | XR_SPACE_LOCATION_POSITION_TRACKED_BIT;
	DummyLocation.pose.orientation = ToXrQuat(FQuat::Identity);
	DummyLocation.pose.position = ToXrVector(FVector::ZeroVector);

	// artifically constrain locations to stage space
	if (!EmulatedReferenceSpaceTypeMap.Contains(BaseSpace) ||
			EmulatedReferenceSpaceTypeMap[BaseSpace] != XR_REFERENCE_SPACE_TYPE_STAGE)
	{
		return DummyLocation;
	}

	int32 PoseIndex = 0;
	if (XRScribePoseReplayMode == EXRScribePoseReplayMode::TimeMatched)
	{
		PoseIndex = GenerateReplaySliceIndexFromTime(Time);
	}
	else
	{
		check(XRScribePoseReplayMode == EXRScribePoseReplayMode::Immediate);
		
		int32 SearchIndex = LastInsertedFrameIndex;
		int32 StopIndex = SearchIndex - PoseHistorySize;
		StopIndex = (StopIndex < 1) ? 1 : StopIndex;
		while (SearchIndex >= StopIndex)
		{
			const int32 HistoryIndex = SearchIndex % PoseHistorySize;
			if (FrameTimeHistory[HistoryIndex].Time == Time)
			{
				break;
			}
		
			SearchIndex--;
		}
		
		if (SearchIndex < StopIndex)
		{
			// We didn't find a match, just use most recent pose
			SearchIndex = LastInsertedFrameIndex;
		}
		
		PoseIndex = FrameTimeHistory[SearchIndex].FrameIndex;
	}
	
	if (EmulatedReferenceSpaceTypeMap.Contains(LocatingSpace) && EmulatedReferenceSpaceTypeMap[LocatingSpace] == XR_REFERENCE_SPACE_TYPE_VIEW)
	{
		const TArray<FOpenXRLocateSpacePacket>& CapturedPoseHistories = ReferencePoseHistories[XR_REFERENCE_SPACE_TYPE_VIEW];
		const int32 CountCapturedPoses = CapturedPoseHistories.Num();
		const int32 CapturedIndex = PoseIndex % CountCapturedPoses;

		return CapturedPoseHistories[CapturedIndex].Location;
	}
	else if (EmulatedActionSpaceNameMap.Contains(LocatingSpace) && ActionPoseHistories.Contains(EmulatedActionSpaceNameMap[LocatingSpace]))
	{
		const FName ActionName = EmulatedActionSpaceNameMap[LocatingSpace];
		const TArray<FOpenXRLocateSpacePacket>& CapturedPoseHistories = ActionPoseHistories[ActionName]; // TODO: should actually check that that we have a history...
		const int32 CountCapturedPoses = CapturedPoseHistories.Num();
		const int32 CapturedIndex = PoseIndex % CountCapturedPoses;

		return CapturedPoseHistories[CapturedIndex].Location;
	}

	return DummyLocation;
}

XrActionStateBoolean FOpenXRActionPoseManager::GetEmulatedActionStateBoolean(const XrActionStateGetInfo* GetInfo, XrTime LastSyncTime)
{
	XrActionStateBoolean EmulatedState{};
	EmulatedState.type = XR_TYPE_ACTION_STATE_BOOLEAN;
	EmulatedState.lastChangeTime = INT64_MAX;

	const FName ActionName = GetEmulatedActionName(GetInfo->action);
	if (BooleanProcessedHistories.Contains(ActionName) &&
		ValidateSubpath(BooleanProcessedHistories[ActionName].ValidSubpaths, GetInfo->subactionPath))
	{
		const int64 SliceIndex = GenerateReplaySliceIndexFromTime(LastSyncTime);

		FBooleanActionState HistoryEntry = BooleanProcessedHistories[ActionName].StateValues[SliceIndex];
		EmulatedState.isActive = HistoryEntry.Active;

		if (EmulatedState.isActive == XR_TRUE)
		{
			EmulatedState.currentState = HistoryEntry.State;

			const XrActionStateBoolean CachedState = GetCachedEmulatedBooleanState(ActionName);
			if (CachedState.isActive == XR_FALSE)
			{
				// TODO: this is the 'first' active entry seen, so we need to just cache it.
				// Can we do this without using this branch?
				EmulatedState.changedSinceLastSync = XR_FALSE;
				EmulatedState.lastChangeTime = LastSyncTime;
				SetCachedEmulatedBooleanState(ActionName, EmulatedState);
			}
			else if (CachedState.currentState != EmulatedState.currentState)
			{
				EmulatedState.changedSinceLastSync = XR_TRUE;
				EmulatedState.lastChangeTime = LastSyncTime;
				SetCachedEmulatedBooleanState(ActionName, EmulatedState);
			}
			else
			{
				EmulatedState = CachedState;
			}
		}
		// TODO: cache off inactive states?
	}
	
	return EmulatedState;
}

XrActionStateFloat FOpenXRActionPoseManager::GetEmulatedActionStateFloat(const XrActionStateGetInfo* GetInfo, XrTime LastSyncTime)
{
	XrActionStateFloat EmulatedState{};
	EmulatedState.type = XR_TYPE_ACTION_STATE_FLOAT;
	EmulatedState.lastChangeTime = INT64_MAX;

	const FName ActionName = GetEmulatedActionName(GetInfo->action);
	if (FloatProcessedHistories.Contains(ActionName) &&
		ValidateSubpath(FloatProcessedHistories[ActionName].ValidSubpaths, GetInfo->subactionPath))
	{
		const int64 SliceIndex = GenerateReplaySliceIndexFromTime(LastSyncTime);

		FFloatActionState HistoryEntry = FloatProcessedHistories[ActionName].StateValues[SliceIndex];
		EmulatedState.isActive = HistoryEntry.Active;

		if (EmulatedState.isActive == XR_TRUE)
		{
			EmulatedState.currentState = HistoryEntry.State;

			const XrActionStateFloat CachedState = GetCachedEmulatedFloatState(ActionName);
			if (CachedState.isActive == XR_FALSE)
			{
				// TODO: this is the 'first' active entry seen, so we need to just cache it.
				// Can we do this without using this branch?
				EmulatedState.changedSinceLastSync = XR_FALSE;
				EmulatedState.lastChangeTime = LastSyncTime;
				SetCachedEmulatedFloatState(ActionName, EmulatedState);
			}
			else if (CachedState.currentState != EmulatedState.currentState)
			{
				EmulatedState.changedSinceLastSync = XR_TRUE;
				EmulatedState.lastChangeTime = LastSyncTime;
				SetCachedEmulatedFloatState(ActionName, EmulatedState);
			}
			else
			{
				EmulatedState = CachedState;
			}
		}
		// TODO: cache off inactive states?
	}

	return EmulatedState;
}

XrActionStateVector2f FOpenXRActionPoseManager::GetEmulatedActionStateVector2f(const XrActionStateGetInfo* GetInfo, XrTime LastSyncTime)
{
	XrActionStateVector2f EmulatedState{};
	EmulatedState.type = XR_TYPE_ACTION_STATE_VECTOR2F;
	EmulatedState.lastChangeTime = INT64_MAX;

	const FName ActionName = GetEmulatedActionName(GetInfo->action);
	if (Vector2fProcessedHistories.Contains(ActionName) &&
		ValidateSubpath(Vector2fProcessedHistories[ActionName].ValidSubpaths, GetInfo->subactionPath))
	{
		const int64 SliceIndex = GenerateReplaySliceIndexFromTime(LastSyncTime);

		FVector2fActionState HistoryEntry = Vector2fProcessedHistories[ActionName].StateValues[SliceIndex];
		EmulatedState.isActive = HistoryEntry.Active;

		if (EmulatedState.isActive == XR_TRUE)
		{
			EmulatedState.currentState = HistoryEntry.State;

			const XrActionStateVector2f CachedState = GetCachedEmulatedVector2fState(ActionName);
			if (CachedState.isActive == XR_FALSE)
			{
				// TODO: this is the 'first' active entry seen, so we need to just cache it.
				// Can we do this without using this branch?
				EmulatedState.changedSinceLastSync = XR_FALSE;
				EmulatedState.lastChangeTime = LastSyncTime;
				SetCachedEmulatedVector2fState(ActionName, EmulatedState);
			}
			else if ((CachedState.currentState.x != EmulatedState.currentState.x) ||
				(CachedState.currentState.y != EmulatedState.currentState.y))
			{
				EmulatedState.changedSinceLastSync = XR_TRUE;
				EmulatedState.lastChangeTime = LastSyncTime;
				SetCachedEmulatedVector2fState(ActionName, EmulatedState);
			}
			else
			{
				EmulatedState = CachedState;
			}
		}
		// TODO: cache off inactive states?
	}

	return EmulatedState;
}

} // namespace UE::XRScribe
