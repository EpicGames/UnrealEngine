// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRScribeEmulationLayer.h"

#include "Containers/MpscQueue.h"
#include "Containers/ArrayView.h"
#include "HAL/Event.h"
#include "HAL/PlatformTime.h"
#include "HAL/UnrealMemory.h"
#include "Logging/LogMacros.h"
#include "Misc/DateTime.h"
#include "Misc/FileHelper.h"
#include "OpenXRCore.h"
#include "RHICommandList.h"
#include "RHIResources.h"
#include "Misc/MessageDialog.h"


// TODO
// * useful error logs at fail points
// * per-api next pointer validation

namespace UE::XRScribe
{

XrResult UnsupportedFuncPlaceholder(EOpenXRAPIPacketId ApiId = EOpenXRAPIPacketId::NumValidAPIPacketIds)
{
	UE_LOGF(LogXRScribeEmulate, Error, "Unimplemented OpenXR API: %d", (uint32)ApiId);
	return XR_ERROR_FUNCTION_UNSUPPORTED;
}

XrTime UETicksToXrTime(int64 Ticks)
{
	return XrTime(Ticks * 100);
}

enum class EUpdateType
{
	SessionState = 0,
	Max,
};

struct FOpenXRUpdateEvent
{
	EUpdateType UpdateType = EUpdateType::Max;

	XrSessionState SessionState = XR_SESSION_STATE_UNKNOWN;

	FOpenXRUpdateEvent() = default;

	FOpenXRUpdateEvent(XrSessionState NewSessionState)
	{
		UpdateType = EUpdateType::SessionState;
		SessionState = NewSessionState;
	}
};

struct FOpenXREmulatedSpace;
using FOpenXREmulatedPath = TStaticArray<ANSICHAR, XR_MAX_PATH_LENGTH>;
struct FOpenXREmulatedActionSet;
struct FOpenXREmulatedAction;
struct FOpenXREmulatedSwapchain;

struct FAnsiStringToPathMapKeyFuncs : TDefaultMapHashableKeyFuncs<const ANSICHAR*, XrPath, false>
{
	static FORCEINLINE bool Matches(KeyInitType A, KeyInitType B)
	{
		return FCStringAnsi::Stricmp(A, B) == 0;
	}

	static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
	{
		return FCrc::Strihash_DEPRECATED(Key);
	}
};

struct FAnsiStringKeyFuncs : DefaultKeyFuncs<const ANSICHAR*>
{
	static FORCEINLINE bool Matches(KeyInitType A, KeyInitType B)
	{
		return FCStringAnsi::Stricmp(A, B) == 0;
	}

	static FORCEINLINE uint32 GetKeyHash(KeyInitType Key)
	{
		return FCrc::Strihash_DEPRECATED(Key);
	}
};

struct FOpenXREmulatedInstance
{
	FOpenXREmulatedInstance()
	{
		FOpenXREmulatedPath NullPathString;
		FCStringAnsi::Strncpy(NullPathString.GetData(), "XR_NULL_PATH", XR_MAX_PATH_LENGTH);
		PathList.Add(MoveTemp(NullPathString));
		ANSIStringToPathMap.Add(NullPathString.GetData(), 0);
	}

	~FOpenXREmulatedInstance() {}

	TArray<TStaticArray<ANSICHAR, XR_MAX_API_LAYER_NAME_SIZE>> Layers;
	TArray<TStaticArray<ANSICHAR, XR_MAX_EXTENSION_NAME_SIZE>> Extensions;

	// TODO: I guess move session pointer here?

	TArray<FOpenXREmulatedPath> PathList;
	TMap<const ANSICHAR*, XrPath, FDefaultSetAllocator, FAnsiStringToPathMapKeyFuncs> ANSIStringToPathMap;

	TArray<TUniquePtr<FOpenXREmulatedActionSet>> ActiveActionSets;
	TSet<const ANSICHAR*, FAnsiStringKeyFuncs> ActiveActionSetNames;
	
	// TODO: is a set more appropriate/performant? Will this list get so big a linear scan would actually hurt?
	// TODO: Maybe the FOpenXREmulatedAction destructor can go into the instance and remove itself from list?
	TArray<XrAction> AllActiveActions; 

	TMap<XrStructureType, XrSystemId> ValidGraphicsBindingTypes;
};

enum class EFrameSubmissionState : uint8
{
	FrameReadyToBegin,
	FrameReadyToFinish,
};

union FOpenXREmulatedGraphicsBinding
{
	XrEventDataBaseHeader Base;
#if defined(XR_USE_GRAPHICS_API_D3D12)
	XrGraphicsBindingD3D12KHR D3D12Binding;
#endif
};

struct FOpenXREmulatedSession
{
	FOpenXREmulatedSession()
	{
		WaitFrameAdvanceEvent = FPlatformProcess::GetSynchEventFromPool(false);
		WaitFrameAdvanceEvent->Trigger();
	}

	~FOpenXREmulatedSession()
	{
		FPlatformProcess::ReturnSynchEventToPool(WaitFrameAdvanceEvent);
		WaitFrameAdvanceEvent = nullptr;
	}

	XrSessionState CurrentSessionState = XR_SESSION_STATE_IDLE;
	XrViewConfigurationType ActiveViewConfigType = XR_VIEW_CONFIGURATION_TYPE_MAX_ENUM;
	
	// OpenXR spec consider a session 'running' between successful calls to xrBeginSession and xrEndSession
	bool bRunning = false;
	bool bExitRequested = false;

	TArray<TUniquePtr<FOpenXREmulatedSpace>> ActiveSpaces;

	FEvent* WaitFrameAdvanceEvent = nullptr;

	TMpscQueue<uint32> WaitFrameFrameCounterPipe;

	EFrameSubmissionState FrameSubmitState = EFrameSubmissionState::FrameReadyToBegin;

	uint32 WaitFrameCounter = 0;
	uint32 BeginFrameCounter = 0;

	// TODO: list of action sets + actions attached
	bool bActionSetsAttached = false;
	TSet<XrActionSet> AttachedActionSets;
	TSet<XrAction> AttachedActions;

	bool bGraphicsBindingConfigured = false;
	FOpenXREmulatedGraphicsBinding GraphicsBinding;

	TArray<TUniquePtr<FOpenXREmulatedSwapchain>> ActiveSwapchains;
};

struct FOpenXREmulatedSpace
{
	enum class ESpaceType
	{
		Reference,
		Action,
	};

	ESpaceType SpaceType;
	XrPosef InitialPose;

	XrReferenceSpaceType RefSpaceType = XR_REFERENCE_SPACE_TYPE_MAX_ENUM;

	XrAction Action = XR_NULL_HANDLE;
	XrPath SubactionPath = XR_NULL_PATH;

	// TODO: constructors from CreateInfo?
};

struct FOpenXREmulatedActionSet
{
	TStaticArray<ANSICHAR, XR_MAX_ACTION_SET_NAME_SIZE> ActionSetName;
	TStaticArray<ANSICHAR, XR_MAX_LOCALIZED_ACTION_SET_NAME_SIZE> LocalizedActionSetName;
	uint32 Priority;

	TArray<TUniquePtr<FOpenXREmulatedAction>> Actions;
	TSet<const ANSICHAR*, FAnsiStringKeyFuncs> ActionNames;
	bool bAttached = false;

	// TODO: do we need a lock for this??
};

struct FOpenXREmulatedAction
{
	FOpenXREmulatedActionSet* OwningActionSet;
	XrActionType ActionType;
	TStaticArray<ANSICHAR, XR_MAX_ACTION_NAME_SIZE> ActionName;
	TStaticArray<ANSICHAR, XR_MAX_LOCALIZED_ACTION_NAME_SIZE> LocalizedActionName;
	TArray<XrPath> SubactionPaths;
};

constexpr uint32 kEmulatedSwapchainLength = 2;

struct FOpenXREmulatedSwapchain
{
	XrSwapchainCreateInfo CreateInfo;
	FString DebugName;
	TStaticArray<FTextureRHIRef, kEmulatedSwapchainLength> TextureList;

	enum class ESwapchainImageAPIState : uint8
	{
		// TODO: might want an UNKNOWN state after creation, because an image must be released by app explicitly before using
		Acquired,
		Waited, // could add an intermediate state if I add in actual waits
		Released,
	};

	struct FImageState
	{
		ESwapchainImageAPIState APIState = ESwapchainImageAPIState::Released;
		int64 AcquireTimeCycles = INT64_MAX;
	};

	TStaticArray<FImageState, kEmulatedSwapchainLength> ImageStateList;

	int32 LabelCounter = 0;
};

FOpenXREmulationLayer::FOpenXREmulationLayer()
{
	// We have to offset the time negatively because the first calls could be fast and fetch the same time!
	StartTimeTicks = FDateTime::Now().GetTicks() - 1e7;

#if defined(XR_USE_GRAPHICS_API_D3D12)
	UE_LOGF(LogXRScribeEmulate, Log, "XRScribe emulation supports D3D12");
#endif
}
FOpenXREmulationLayer::~FOpenXREmulationLayer() = default;

bool FOpenXREmulationLayer::SupportsInstanceExtension(const ANSICHAR* ExtensionName)
{
	FReadScopeLock InstanceLock(InstanceMutex);
	return CurrentInstance->Extensions.ContainsByPredicate([ExtensionName](TStaticArray<ANSICHAR, XR_MAX_EXTENSION_NAME_SIZE>& EnabledExtension)
	{
		return FCStringAnsi::Strcmp(EnabledExtension.GetData(), ExtensionName) == 0;
	});
}

bool FOpenXREmulationLayer::LoadCaptureFromFile(const FString& EmulationLoadPath)
{
	if (FFileHelper::LoadFileToArray(CaptureDecoder.GetEncodedData(), *EmulationLoadPath, EFileRead::FILEREAD_Silent))
	{
		UE_LOGF(LogXRScribeEmulate, Log, "Capture successfully loaded: %ls", *EmulationLoadPath);
		if (CaptureDecoder.DecodeDataFromMemory())
		{
			PostLoadActions();
			return true;
		}
		else
		{
			UE_LOGF(LogXRScribeEmulate, Error, "Failed to decode capture");

			// On Windows, a workaround in FOpenXRHMDModule::PreInit means we may end up initializing XRScribe twice
			// For now we'll always load the same file, if it's invalid we should only display an error message once
			static bool bDisplayedErrorMessage = false;

			if (!bDisplayedErrorMessage)
			{
				FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, FText::FromString(FString("XRScribe capture at ") + EmulationLoadPath + FString(" was malformed or captured with a different version of the plugin. Emulation layer will be disabled.\n\nSee log for details.")));
				bDisplayedErrorMessage = true;
			}
			
			// Clean up loaded data
			CaptureDecoder.GetEncodedData().Empty();
			return false;
		}
	}
	else
	{
		UE_LOGF(LogXRScribeEmulate, Log, "Capture '%ls' failed to load.", *EmulationLoadPath);
		return false;
	}
}

bool FOpenXREmulationLayer::LoadCaptureFromData(const TArray<uint8>& EncodedData)
{
	CaptureDecoder.GetEncodedData().Append(EncodedData);

	if (CaptureDecoder.DecodeDataFromMemory())
	{
		PostLoadActions();
		return true;
	}
	else
	{
		CaptureDecoder.GetEncodedData().Empty();
		return false;
	}
}

void FOpenXREmulationLayer::PostLoadActions()
{
	// TODO: Derive _actually_ supported layers (probably none?)

	TArray<TStaticArray<ANSICHAR, XR_MAX_API_LAYER_NAME_SIZE>> EmulatedLayers;
	SupportedEmulatedLayers = CaptureDecoder.GetApiLayerProperties().FilterByPredicate([EmulatedLayers](const XrApiLayerProperties& Layer) {
		for (const TStaticArray<ANSICHAR, XR_MAX_API_LAYER_NAME_SIZE>& EmulatedLayerName : EmulatedLayers) //-V1078
		{
			if (FCStringAnsi::Strcmp(Layer.layerName, EmulatedLayerName.GetData()) == 0)
			{
				return true;
			}
		}
		return false;
		});

	// TODO: Derive supported extensions
	TArray<TStaticArray<ANSICHAR, XR_MAX_EXTENSION_NAME_SIZE>> EmulatedExtensions;
	//FCStringAnsi::Strncpy(EmulatedExtensions.AddZeroed_GetRef().GetData(), XR_KHR_D3D11_ENABLE_EXTENSION_NAME, XR_MAX_EXTENSION_NAME_SIZE);
#if defined(XR_USE_GRAPHICS_API_D3D12)
	FCStringAnsi::Strncpy(EmulatedExtensions.AddZeroed_GetRef().GetData(), XR_KHR_D3D12_ENABLE_EXTENSION_NAME, XR_MAX_EXTENSION_NAME_SIZE);
#endif
	//FCStringAnsi::Strncpy(EmulatedExtensions.AddZeroed_GetRef().GetData(), XR_KHR_OPENGL_ENABLE_EXTENSION_NAME, XR_MAX_EXTENSION_NAME_SIZE);
	//FCStringAnsi::Strncpy(EmulatedExtensions.AddZeroed_GetRef().GetData(), XR_KHR_OPENGL_ES_ENABLE_EXTENSION_NAME, XR_MAX_EXTENSION_NAME_SIZE);
	//FCStringAnsi::Strncpy(EmulatedExtensions.AddZeroed_GetRef().GetData(), XR_KHR_VULKAN_ENABLE_EXTENSION_NAME, XR_MAX_EXTENSION_NAME_SIZE);
	//FCStringAnsi::Strncpy(EmulatedExtensions.AddZeroed_GetRef().GetData(), XR_KHR_VULKAN_SWAPCHAIN_FORMAT_LIST_EXTENSION_NAME, XR_MAX_EXTENSION_NAME_SIZE);
	//FCStringAnsi::Strncpy(EmulatedExtensions.AddZeroed_GetRef().GetData(), XR_KHR_COMPOSITION_LAYER_DEPTH_EXTENSION_NAME, XR_MAX_EXTENSION_NAME_SIZE);
	//FCStringAnsi::Strncpy(EmulatedExtensions.AddZeroed_GetRef().GetData(), XR_VARJO_QUAD_VIEWS_EXTENSION_NAME, XR_MAX_EXTENSION_NAME_SIZE);
	//FCStringAnsi::Strncpy(EmulatedExtensions.AddZeroed_GetRef().GetData(), XR_KHR_VISIBILITY_MASK_EXTENSION_NAME, XR_MAX_EXTENSION_NAME_SIZE);
	//FCStringAnsi::Strncpy(EmulatedExtensions.AddZeroed_GetRef().GetData(), XR_KHR_BINDING_MODIFICATION_EXTENSION_NAME, XR_MAX_EXTENSION_NAME_SIZE);
	//FCStringAnsi::Strncpy(EmulatedExtensions.AddZeroed_GetRef().GetData(), XR_EPIC_VIEW_CONFIGURATION_FOV_EXTENSION_NAME, XR_MAX_EXTENSION_NAME_SIZE);
	//FCStringAnsi::Strncpy(EmulatedExtensions.AddZeroed_GetRef().GetData(), "XR_EXT_dpad_binding", XR_MAX_EXTENSION_NAME_SIZE);
	//FCStringAnsi::Strncpy(EmulatedExtensions.AddZeroed_GetRef().GetData(), "XR_EXT_active_action_set_priority", XR_MAX_EXTENSION_NAME_SIZE);

	// TODO: Log all this state at load time, for debugging!

	SupportedEmulatedExtensions = CaptureDecoder.GetInstanceExtensionProperties().FilterByPredicate([EmulatedExtensions](const XrExtensionProperties& Extension) {
		for (const TStaticArray<ANSICHAR, XR_MAX_EXTENSION_NAME_SIZE>& EmulatedExtensionName : EmulatedExtensions)
		{
			if (FCStringAnsi::Strcmp(Extension.extensionName, EmulatedExtensionName.GetData()) == 0)
			{
				return true;
			}
		}
		return false;
		});

	// Append "Emulated" to runtime name and system name
	const char* EmuStringAddend = " (Emulated)";

	EmulatedInstanceProperties = CaptureDecoder.GetInstanceProperties();
	{
		const int32 ActualRuntimeNameLen = FCStringAnsi::Strlen(EmulatedInstanceProperties.runtimeName);
		const int32 RemainingSpace = XR_MAX_RUNTIME_NAME_SIZE - ActualRuntimeNameLen;
		FCStringAnsi::Strncpy(&EmulatedInstanceProperties.runtimeName[ActualRuntimeNameLen], EmuStringAddend, RemainingSpace);
	}

	EmulatedSystemProperties = CaptureDecoder.GetSystemProperties();

	// TODO: Support layers inside emulation
	EmulatedSystemProperties.graphicsProperties.maxLayerCount = 1;
	{
		const int32 ActualSystemNameLen = FCStringAnsi::Strlen(EmulatedSystemProperties.systemName);
		const int32 RemainingSpace = XR_MAX_SYSTEM_NAME_SIZE - ActualSystemNameLen;
		FCStringAnsi::Strncpy(&EmulatedSystemProperties.systemName[ActualSystemNameLen], EmuStringAddend, RemainingSpace);
	}

	ActionPoseManager.RegisterCapturedPathStrings(CaptureDecoder.GetPathToStringMap());
	ActionPoseManager.RegisterCapturedWaitFrames(CaptureDecoder.GetWaitFrames());

	ActionPoseManager.RegisterCapturedReferenceSpaces(CaptureDecoder.GetCreatedReferenceSpaces());

	ActionPoseManager.RegisterCapturedActions(CaptureDecoder.GetCreatedActions());
	ActionPoseManager.RegisterCapturedActionSpaces(CaptureDecoder.GetCreatedActionSpaces());

	ActionPoseManager.RegisterCapturedSpaceHistories(CaptureDecoder.GetSpaceLocations());

	ActionPoseManager.RegisterCapturedActionStates(CaptureDecoder.GetSyncActions(),
		CaptureDecoder.GetBooleanActionStates(), 
		CaptureDecoder.GetFloatActionStates(), 
		CaptureDecoder.GetVectorActionStates(), 
		CaptureDecoder.GetPoseActionStates());

	ActionPoseManager.ProcessCapturedHistories();
}

bool FOpenXREmulationLayer::InstanceHandleCheck(XrInstance InputInstance)
{
	if ((InputInstance == XR_NULL_HANDLE) ||
		(InputInstance != reinterpret_cast<XrInstance>(CurrentInstance.Get())))
	{
		return false;
	}

	return true;
}

bool FOpenXREmulationLayer::SystemIdCheck(XrSystemId InputSystemId)
{
	if ((InputSystemId == XR_NULL_SYSTEM_ID) ||
		(InputSystemId != MagicSystemId))
	{
		return false;
	}

	return true;
}

bool FOpenXREmulationLayer::SessionHandleCheck(XrSession InputSession)
{
	if ((InputSession == XR_NULL_HANDLE) ||
		(InputSession != reinterpret_cast<XrSession>(CurrentSession.Get())))
	{
		UE_LOGF(LogXRScribeEmulate, Error, "Session handle invalid: %llx", reinterpret_cast<intptr_t>(InputSession));
		return false;
	}

	return true;
}

bool FOpenXREmulationLayer::SpaceHandleCheck(XrSpace InputSpace)
{
	if (!CurrentSession.IsValid())
	{
		return false;
	}

	FReadScopeLock SessionLock(SessionMutex);
	return CurrentSession->ActiveSpaces.ContainsByPredicate([InputSpace](const TUniquePtr<FOpenXREmulatedSpace>& EmulatedSpace)
	{
		return InputSpace == reinterpret_cast<XrSpace>(EmulatedSpace.Get());
	});
}

bool FOpenXREmulationLayer::SwapchainHandleCheck(XrSwapchain InputSwapchain)
{
	if (!CurrentSession.IsValid())
	{
		return false;
	}

	FReadScopeLock SessionLock(SessionMutex);
	return CurrentSession->ActiveSwapchains.ContainsByPredicate([InputSwapchain](const TUniquePtr<FOpenXREmulatedSwapchain>& EmulatedSwapchain)
	{
		return InputSwapchain == reinterpret_cast<XrSwapchain>(EmulatedSwapchain.Get());
	});
}

bool FOpenXREmulationLayer::ActionSetHandleCheck(XrActionSet InputActionSet)
{
	if (!CurrentInstance.IsValid())
	{
		return false;
	}

	FReadScopeLock InstanceLock(InstanceMutex);
	return CurrentInstance->ActiveActionSets.ContainsByPredicate([InputActionSet](const TUniquePtr<FOpenXREmulatedActionSet>& EmulatedActionSet)
	{
		return InputActionSet == reinterpret_cast<XrActionSet>(EmulatedActionSet.Get());
	});
}

bool FOpenXREmulationLayer::ActionHandleCheck(XrAction InputAction)
{
	if (!CurrentInstance.IsValid())
	{
		return false;
	}

	FReadScopeLock InstanceLock(InstanceMutex);
	return CurrentInstance->AllActiveActions.Contains(InputAction);
}

bool FOpenXREmulationLayer::IsCurrentSessionRunning()
{
	FReadScopeLock SessionLock(SessionMutex);
	return CurrentSession->bRunning;
}

bool ValidateSessionStateTransition(XrSessionState BeforeState, XrSessionState AfterState)
{
	if (AfterState == XR_SESSION_STATE_LOSS_PENDING)
	{
		// When would we generate this??
		// TODO: log this
		return true;
	}
	
	if (BeforeState == XR_SESSION_STATE_IDLE)
	{
		if ((AfterState == XR_SESSION_STATE_READY) ||
			(AfterState == XR_SESSION_STATE_EXITING))
		{
			return true;
		}
	}
	else if (BeforeState == XR_SESSION_STATE_READY)
	{
		if ((AfterState == XR_SESSION_STATE_IDLE) ||
			(AfterState == XR_SESSION_STATE_SYNCHRONIZED))
		{
			return true;
		}
	}
	else if (BeforeState == XR_SESSION_STATE_SYNCHRONIZED)
	{
		if ((AfterState == XR_SESSION_STATE_VISIBLE) ||
			(AfterState == XR_SESSION_STATE_STOPPING))
		{
			return true;
		}
	}
	else if (BeforeState == XR_SESSION_STATE_VISIBLE)
	{
		if ((AfterState == XR_SESSION_STATE_FOCUSED) ||
			(AfterState == XR_SESSION_STATE_SYNCHRONIZED))
		{
			return true;
		}
	}
	else if (BeforeState == XR_SESSION_STATE_FOCUSED)
	{
		if (AfterState == XR_SESSION_STATE_VISIBLE)
		{
			return true;
		}
	}
	else if (BeforeState == XR_SESSION_STATE_STOPPING)
	{
		if (AfterState == XR_SESSION_STATE_IDLE)
		{
			return true;
		}
	}
	else
	{
		check(0);
	}

	return false;
}

void FOpenXREmulationLayer::UpdateSessionState(FOpenXREmulatedSession& Session, XrSessionState AfterState)
{
	// Write lock should be acquired already
	check(SessionMutex.TryWriteLock() == false);

	if (!ValidateSessionStateTransition(Session.CurrentSessionState, AfterState))
	{
		// TODO: log error
		check(0);
	}
	Session.CurrentSessionState = AfterState;
}

XrResult FOpenXREmulationLayer::SetupGraphicsBinding(const XrSystemId SystemId, const void* InBinding, TUniquePtr<FOpenXREmulatedSession>& Session)
{
	if (InBinding == nullptr)
	{
		// TODO: headless valid here
		return XR_ERROR_GRAPHICS_DEVICE_INVALID;
	}

	// We use XrEventDataBaseHeader to just check the type, nothing else. There's
	// no 'default' structure to just check the type
	const XrEventDataBaseHeader* TypeCheckStruct = reinterpret_cast<const XrEventDataBaseHeader*>(InBinding);

	XrStructureType RequestedType = TypeCheckStruct->type;
#if defined(XR_USE_GRAPHICS_API_D3D12)

	if (RequestedType == XR_TYPE_GRAPHICS_BINDING_D3D12_KHR)
	{
		UE_LOGF(LogXRScribeEmulate, Log, "Session requesting D3D12 graphics binding");
	}
	else
#endif
	{
		// TODO: support the other binding types
		return XR_ERROR_GRAPHICS_DEVICE_INVALID;
	}


	{
		FReadScopeLock InstanceLock(InstanceMutex);
		if ((!CurrentInstance->ValidGraphicsBindingTypes.Contains(RequestedType)) ||
			(CurrentInstance->ValidGraphicsBindingTypes[RequestedType] != SystemId))
		{
			UE_LOGF(LogXRScribeEmulate, Error, "GraphicsRequirement call missing");
			return XR_ERROR_GRAPHICS_REQUIREMENTS_CALL_MISSING;
		}
	}

#if defined(XR_USE_GRAPHICS_API_D3D12)
	if (RequestedType == XR_TYPE_GRAPHICS_BINDING_D3D12_KHR)
	{
		Session->GraphicsBinding.D3D12Binding = *reinterpret_cast<const XrGraphicsBindingD3D12KHR*>(InBinding);
	}
#endif
	Session->bGraphicsBindingConfigured = true; // unless headless

	return XR_SUCCESS;
}

// TODO: Drive the formats here from GPixelFormats instead of the brittle manual mapping
// On some platforms, this will mean removing UNORM/SRGB from the types, to get to the
// underlying format
EPixelFormat FOpenXREmulationLayer::ConvertPlatformFormat(int64 PlatformFormat)
{
	XrStructureType BindingType = XR_TYPE_UNKNOWN;
	{
		FReadScopeLock SessionLock(SessionMutex);
		BindingType = CurrentSession->GraphicsBinding.Base.type;
	}

	EPixelFormat SelectedFormat = EPixelFormat::PF_Unknown;

#if defined(XR_USE_GRAPHICS_API_D3D12)
	if (BindingType == XR_TYPE_GRAPHICS_BINDING_D3D12_KHR)
	{
		static const TMap<DXGI_FORMAT, EPixelFormat> DX12PlatformToEngineFormat =
		{
			{DXGI_FORMAT_R10G10B10A2_UNORM, PF_A2B10G10R10},
			{DXGI_FORMAT_R8G8B8A8_UNORM_SRGB, PF_R8G8B8A8},
			{DXGI_FORMAT_R8G8B8A8_UNORM, PF_R8G8B8A8},
			{DXGI_FORMAT_B8G8R8A8_UNORM_SRGB, PF_B8G8R8A8},
			{DXGI_FORMAT_B8G8R8A8_UNORM, PF_B8G8R8A8},
			{DXGI_FORMAT_R24G8_TYPELESS, PF_DepthStencil},
			{DXGI_FORMAT_D24_UNORM_S8_UINT, PF_DepthStencil},
			{DXGI_FORMAT_R32G8X24_TYPELESS, PF_DepthStencil},
			{DXGI_FORMAT_D32_FLOAT_S8X24_UINT, PF_DepthStencil},
		};

		DXGI_FORMAT DxgiFormat = static_cast<DXGI_FORMAT>(PlatformFormat);

		if (DX12PlatformToEngineFormat.Contains(DxgiFormat))
		{
			SelectedFormat = DX12PlatformToEngineFormat[DxgiFormat];
			UE_LOGF(LogXRScribeEmulate, Log, "DXGI_FORMAT %d selects EPixelFormat for swapchain: %d", DxgiFormat, SelectedFormat);
		}
		else
		{
			UE_LOGF(LogXRScribeEmulate, Error, "Unknown DXGI_FORMAT requested for swapchain: %d", DxgiFormat);
		}
	}
#endif

	return SelectedFormat;
}

// static
ETextureCreateFlags FOpenXREmulationLayer::ConvertXrSwapchainFlagsToTextureCreateFlags(const XrSwapchainCreateInfo* CreateInfo)
{
	ETextureCreateFlags TextureCreateFlags = ETextureCreateFlags::None;
	if ((CreateInfo->usageFlags & XR_SWAPCHAIN_USAGE_UNORDERED_ACCESS_BIT) != 0)
	{
		TextureCreateFlags |= ETextureCreateFlags::UAV;
	}
	if ((CreateInfo->usageFlags & XR_SWAPCHAIN_USAGE_SAMPLED_BIT) != 0)
	{
		TextureCreateFlags |= ETextureCreateFlags::ShaderResource;
	}
	if ((CreateInfo->usageFlags & XR_SWAPCHAIN_USAGE_INPUT_ATTACHMENT_BIT_KHR) != 0)
	{
		TextureCreateFlags |= ETextureCreateFlags::InputAttachmentRead;
	}
	if ((CreateInfo->usageFlags & XR_SWAPCHAIN_USAGE_MUTABLE_FORMAT_BIT) == 0)
	{
		// See FOpenXRSwapchain::CreateSwapchain for our reasoning
		// If we have SRGB requested on the UE side, we don't request mutable format
		TextureCreateFlags |= ETextureCreateFlags::SRGB;
	}
	if ((CreateInfo->usageFlags & XR_SWAPCHAIN_USAGE_COLOR_ATTACHMENT_BIT) != 0)
	{
		// I can't mix target + resolve because D3D12Texture complains. Not sure if complaint is valid
		//TextureCreateFlags |= ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ResolveTargetable;
		TextureCreateFlags |= ETextureCreateFlags::ResolveTargetable;
	}
	else if ((CreateInfo->usageFlags & XR_SWAPCHAIN_USAGE_DEPTH_STENCIL_ATTACHMENT_BIT) != 0)
	{
		// TODO: validate the depth extension is active

		// I can't mix target + resolve because D3D12Texture complains. Not sure if complaint is valid
		//TextureCreateFlags |= ETextureCreateFlags::DepthStencilTargetable | ETextureCreateFlags::DepthStencilResolveTarget | ETextureCreateFlags::InputAttachmentRead;
		TextureCreateFlags |= ETextureCreateFlags::DepthStencilResolveTarget | ETextureCreateFlags::InputAttachmentRead;

	}

	return TextureCreateFlags;
}

XrResult FOpenXREmulationLayer::XrLayerEnumerateApiLayerProperties(uint32_t propertyCapacityInput, uint32_t* propertyCountOutput, XrApiLayerProperties* properties)
{
	if (propertyCapacityInput == 0)
	{
		check(propertyCountOutput != nullptr);
		*propertyCountOutput = SupportedEmulatedLayers.Num();
	}
	else
	{
		check(properties != nullptr);

		const uint32 NumPropertiesToCopy = FMath::Min(propertyCapacityInput, (uint32)SupportedEmulatedLayers.Num());
		*propertyCountOutput = NumPropertiesToCopy;

		FMemory::Memcpy(properties, SupportedEmulatedLayers.GetData(), NumPropertiesToCopy * sizeof(XrApiLayerProperties));

	}
	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerEnumerateInstanceExtensionProperties(const char* layerName, uint32_t propertyCapacityInput, uint32_t* propertyCountOutput, XrExtensionProperties* properties)
{
	if (layerName != nullptr)
	{
		// TODO: log the failure, we don't stash this info anywhere
		return XR_ERROR_RUNTIME_FAILURE;
	}

	if (propertyCapacityInput == 0)
	{
		check(propertyCountOutput != nullptr);
		*propertyCountOutput = SupportedEmulatedExtensions.Num();
	}
	else
	{
		check(properties != nullptr);

		const uint32 NumPropertiesToCopy = FMath::Min(propertyCapacityInput, (uint32)SupportedEmulatedExtensions.Num());
		*propertyCountOutput = NumPropertiesToCopy;

		FMemory::Memcpy(properties, SupportedEmulatedExtensions.GetData(), NumPropertiesToCopy * sizeof(XrExtensionProperties));

	}

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerCreateInstance(const XrInstanceCreateInfo* createInfo, XrInstance* instance)
{
	if ((createInfo == nullptr) ||
		(createInfo->type != XR_TYPE_INSTANCE_CREATE_INFO) ||
		(createInfo->createFlags != 0))
	{
		// log error
		return XR_ERROR_VALIDATION_FAILURE;
	}

	{
		FReadScopeLock InstanceLock(InstanceMutex);
		if (CurrentInstance.IsValid())
		{
			// TODO: log failure, since we only manage one instance
			return XR_ERROR_RUNTIME_FAILURE;
		}
	}


	// TODO: do we need the application info?
	// TODO: Handle structs in next
	// TODO: Validate layers + extensions are correct

	{
		FWriteScopeLock InstanceLock(InstanceMutex);

		CurrentInstance = MakeUnique<FOpenXREmulatedInstance>();
		CurrentInstance->Layers.AddDefaulted(createInfo->enabledApiLayerCount);
		for (uint32 ApiLayerIndex = 0; ApiLayerIndex < createInfo->enabledApiLayerCount; ApiLayerIndex++)
		{
			FCStringAnsi::Strncpy(
				CurrentInstance->Layers[ApiLayerIndex].GetData(),
				createInfo->enabledApiLayerNames[ApiLayerIndex],
				XR_MAX_API_LAYER_NAME_SIZE);
		}

		CurrentInstance->Extensions.AddDefaulted(createInfo->enabledExtensionCount);
		for (uint32 ExtensionIndex = 0; ExtensionIndex < createInfo->enabledExtensionCount; ExtensionIndex++)
		{
			FCStringAnsi::Strncpy(
				CurrentInstance->Extensions[ExtensionIndex].GetData(),
				createInfo->enabledExtensionNames[ExtensionIndex],
				XR_MAX_EXTENSION_NAME_SIZE);
		}

		*instance = reinterpret_cast<XrInstance>(CurrentInstance.Get());
	}

	return XR_SUCCESS;
}

// Instance
XrResult FOpenXREmulationLayer::XrLayerDestroyInstance(XrInstance instance)
{
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	CurrentInstance.Reset();

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerGetInstanceProperties(XrInstance instance, XrInstanceProperties* instanceProperties)
{
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((instanceProperties == nullptr) ||
		(instanceProperties->type != XR_TYPE_INSTANCE_PROPERTIES))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	*instanceProperties = EmulatedInstanceProperties;
	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerPollEvent(XrInstance instance, XrEventDataBuffer* eventData)
{
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((eventData == nullptr) ||
		(eventData->type != XR_TYPE_EVENT_DATA_BUFFER))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	// TODO: if we wanted an event processing thread, we would add _another_ queue
	// that the other thread would pump. That thread would hand events back to this event queue
	// We might want to process session state changes once per frame?

	XrResult Result = XR_EVENT_UNAVAILABLE;
	if (!PendingApplicationUpdateQueue.IsEmpty())
	{
		Result = XR_ERROR_RUNTIME_FAILURE;

		FOpenXRUpdateEvent EventForApp;
		PendingApplicationUpdateQueue.Dequeue(EventForApp);

		if (EventForApp.UpdateType == EUpdateType::SessionState)
		{
			XrEventDataSessionStateChanged& EventDataSessionStateChanged =
				reinterpret_cast<XrEventDataSessionStateChanged&>(*eventData);

			check(EventForApp.SessionState != XR_SESSION_STATE_UNKNOWN);

			EventDataSessionStateChanged.type = XR_TYPE_EVENT_DATA_SESSION_STATE_CHANGED;
			EventDataSessionStateChanged.state = EventForApp.SessionState;
			EventDataSessionStateChanged.time = UETicksToXrTime(FDateTime::Now().GetTicks() - StartTimeTicks);

			{
				check(CurrentSession.IsValid());
				EventDataSessionStateChanged.session = reinterpret_cast<XrSession>(CurrentSession.Get());
			}

			Result = XR_SUCCESS;
		}
		else
		{
			check(0);
		}

	}
	
	return Result;
}

	
XrResult FOpenXREmulationLayer::XrLayerResultToString(XrInstance instance, XrResult value, char buffer[XR_MAX_RESULT_STRING_SIZE]) { return UnsupportedFuncPlaceholder(EOpenXRAPIPacketId::ResultToString); }
XrResult FOpenXREmulationLayer::XrLayerStructureTypeToString(XrInstance instance, XrStructureType value, char buffer[XR_MAX_STRUCTURE_NAME_SIZE]) { return UnsupportedFuncPlaceholder(EOpenXRAPIPacketId::StructureTypeToString); }

XrResult FOpenXREmulationLayer::XrLayerGetSystem(XrInstance instance, const XrSystemGetInfo* getInfo, XrSystemId* systemId)
{ 
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((getInfo == nullptr) ||
		(getInfo->type != XR_TYPE_SYSTEM_GET_INFO) ||
		(getInfo->formFactor != XR_FORM_FACTOR_HEAD_MOUNTED_DISPLAY) ||
		(systemId == nullptr))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	*systemId = MagicSystemId;
	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerGetSystemProperties(XrInstance instance, XrSystemId systemId, XrSystemProperties* properties)
{ 
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if (!SystemIdCheck(systemId))
	{
		return XR_ERROR_SYSTEM_INVALID;
	}

	if ((properties == nullptr) ||
		(properties->type != XR_TYPE_SYSTEM_PROPERTIES))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	*properties = EmulatedSystemProperties;
	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerEnumerateEnvironmentBlendModes(XrInstance instance, XrSystemId systemId, XrViewConfigurationType viewConfigurationType, uint32_t environmentBlendModeCapacityInput, uint32_t* environmentBlendModeCountOutput, XrEnvironmentBlendMode* environmentBlendModes)
{
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if (!SystemIdCheck(systemId))
	{
		return XR_ERROR_SYSTEM_INVALID;
	}

	if (viewConfigurationType != XR_VIEW_CONFIGURATION_TYPE_PRIMARY_STEREO)
	{
		// TODO: handle other view config types?
		return XR_ERROR_RUNTIME_FAILURE;
	}

	if (environmentBlendModeCountOutput == nullptr)
	{
		// log error
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (environmentBlendModeCapacityInput == 0)
	{
		*environmentBlendModeCountOutput = CaptureDecoder.GetEnvironmentBlendModes().Num();
	}
	else
	{
		if (environmentBlendModes == nullptr)
		{
			// log error
			return XR_ERROR_VALIDATION_FAILURE;
		}

		const uint32 NumBlendModesToCopy = FMath::Min(environmentBlendModeCapacityInput, (uint32)CaptureDecoder.GetEnvironmentBlendModes().Num());
		*environmentBlendModeCountOutput = NumBlendModesToCopy;

		FMemory::Memcpy(environmentBlendModes, CaptureDecoder.GetEnvironmentBlendModes().GetData(), NumBlendModesToCopy * sizeof(*environmentBlendModes));

	}

	return XrResult::XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerCreateSession(XrInstance instance, const XrSessionCreateInfo* createInfo, XrSession* session)
{
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((createInfo == nullptr) ||
		(createInfo->type != XR_TYPE_SESSION_CREATE_INFO) ||
		(createInfo->createFlags != 0))
	{
		// log error
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (!SystemIdCheck(createInfo->systemId))
	{
		return XR_ERROR_SYSTEM_INVALID;
	}

	{
		FReadScopeLock SessionLock(SessionMutex);
		if (CurrentSession.IsValid())
		{
			// TODO: log failure, since we only manage one session
			return XR_ERROR_RUNTIME_FAILURE;
		}
	}

	// TODO: validate graphics binding
	TUniquePtr<FOpenXREmulatedSession> NewSession = MakeUnique<FOpenXREmulatedSession>();

	const XrResult BindingResult = SetupGraphicsBinding(createInfo->systemId, createInfo->next, NewSession);
	if (BindingResult != XR_SUCCESS)
	{
		return BindingResult;
	}

	{
		FWriteScopeLock SessionLock(SessionMutex);
		CurrentSession = MoveTemp(NewSession);
		*session = reinterpret_cast<XrSession>(CurrentSession.Get());
		UpdateSessionState(*CurrentSession, XR_SESSION_STATE_READY);
	}

	PendingApplicationUpdateQueue.Enqueue(FOpenXRUpdateEvent(XR_SESSION_STATE_READY));

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerDestroySession(XrSession session)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	// from the spec: "The application is responsible for ensuring that it has no calls using session in progress when the session is destroyed."
	CurrentSession.Reset();

	ActionPoseManager.OnSessionTeardown();

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerEnumerateReferenceSpaces(XrSession session, uint32_t spaceCapacityInput, uint32_t* spaceCountOutput, XrReferenceSpaceType* spaces)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if (spaceCountOutput == nullptr)
	{
		// log error
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (spaceCapacityInput == 0)
	{
		*spaceCountOutput = CaptureDecoder.GetReferenceSpaceTypes().Num();
	}
	else
	{
		if (spaces == nullptr)
		{
			// log error
			return XR_ERROR_VALIDATION_FAILURE;
		}

		const uint32 NumSpacesToCopy = FMath::Min(spaceCapacityInput, (uint32)CaptureDecoder.GetReferenceSpaceTypes().Num());
		*spaceCountOutput = NumSpacesToCopy;

		FMemory::Memcpy(spaces, CaptureDecoder.GetReferenceSpaceTypes().GetData(), NumSpacesToCopy * sizeof(*spaces));

	}

	return XrResult::XR_SUCCESS;
}
XrResult FOpenXREmulationLayer::XrLayerCreateReferenceSpace(XrSession session, const XrReferenceSpaceCreateInfo* createInfo, XrSpace* space)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((createInfo == nullptr) || (space == nullptr))
	{
		UE_LOGF(LogXRScribeEmulate, Error, "CreateReferenceSpace: nullptr arguments");
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (createInfo->type != XR_TYPE_REFERENCE_SPACE_CREATE_INFO)
	{
		UE_LOGF(LogXRScribeEmulate, Error, "CreateReferenceSpace: invalid XrStructureType for XrReferenceSpaceCreateInfo");
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (!CaptureDecoder.GetReferenceSpaceTypes().Contains(createInfo->referenceSpaceType))
	{
		UE_LOGF(LogXRScribeEmulate, Error, "CreateReferenceSpace: unsupported XrReferenceSpaceType");
		return XR_ERROR_REFERENCE_SPACE_UNSUPPORTED;
	}

	{
		const FQuat QuatCheck = ToFQuat(createInfo->poseInReferenceSpace.orientation);
		if (!QuatCheck.IsNormalized())
		{
			UE_LOGF(LogXRScribeEmulate, Error, "CreateReferenceSpace: invalid reference pose");
			return XR_ERROR_POSE_INVALID;
		}
	}

	TUniquePtr<FOpenXREmulatedSpace> CreatedReferenceSpace = MakeUnique<FOpenXREmulatedSpace>();
	CreatedReferenceSpace->SpaceType = FOpenXREmulatedSpace::ESpaceType::Reference;
	CreatedReferenceSpace->InitialPose = createInfo->poseInReferenceSpace;
	CreatedReferenceSpace->RefSpaceType = createInfo->referenceSpaceType;

	{
		FWriteScopeLock SessionLock(SessionMutex);
		CurrentSession->ActiveSpaces.Add(MoveTemp(CreatedReferenceSpace));
		*space = reinterpret_cast<XrSpace>(CurrentSession->ActiveSpaces.Last().Get());
	}

	ActionPoseManager.RegisterEmulatedReferenceSpace(*createInfo, *space);

	return XrResult::XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerGetReferenceSpaceBoundsRect(XrSession session, XrReferenceSpaceType referenceSpaceType, XrExtent2Df* bounds)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if (bounds == nullptr)
	{
		UE_LOGF(LogXRScribeEmulate, Error, "GetReferenceSpaceBoundsRect: nullptr bounds");
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (!CaptureDecoder.GetReferenceSpaceTypes().Contains(referenceSpaceType))
	{
		UE_LOGF(LogXRScribeEmulate, Error, "GetReferenceSpaceBoundsRect: unsupported XrReferenceSpaceType");
		return XR_ERROR_REFERENCE_SPACE_UNSUPPORTED;
	}

	const TMap<XrReferenceSpaceType, XrExtent2Df>& BoundsMap = CaptureDecoder.GetReferenceSpaceBounds();
	if (!BoundsMap.Contains(referenceSpaceType))
	{
		// TODO: maybe we return dummy bounds instead? Could even be configurable!
		UE_LOGF(LogXRScribeEmulate, Error, "GetReferenceSpaceBoundsRect: unknown bounds for supported reference space");
		return XR_ERROR_RUNTIME_FAILURE;
	}

	*bounds = BoundsMap[referenceSpaceType];

	return XrResult::XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerCreateActionSpace(XrSession session, const XrActionSpaceCreateInfo* createInfo, XrSpace* space)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((createInfo == nullptr) || (space == nullptr))
	{
		UE_LOGF(LogXRScribeEmulate, Error, "CreateActionSpace: nullptr arguments");
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (createInfo->type != XR_TYPE_ACTION_SPACE_CREATE_INFO)
	{
		UE_LOGF(LogXRScribeEmulate, Error, "CreateActionSpace: invalid XrStructureType for XrActionSpaceCreateInfo");
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (!ActionHandleCheck(createInfo->action))
	{
		UE_LOGF(LogXRScribeEmulate, Error, "CreateActionSpace: unknown action");
		return XR_ERROR_VALIDATION_FAILURE;
	}

	// TODO verify action as XR_ACTION_TYPE_POSE_INPUT
	// TODO Validate path

	{
		const FQuat QuatCheck = ToFQuat(createInfo->poseInActionSpace.orientation);
		if (!QuatCheck.IsNormalized())
		{
			UE_LOGF(LogXRScribeEmulate, Error, "CreateActionSpace: invalid pose");
			return XR_ERROR_POSE_INVALID;
		}
	}

	TUniquePtr<FOpenXREmulatedSpace> CreatedActionSpace = MakeUnique<FOpenXREmulatedSpace>();
	CreatedActionSpace->SpaceType = FOpenXREmulatedSpace::ESpaceType::Action;
	CreatedActionSpace->InitialPose = createInfo->poseInActionSpace;
	CreatedActionSpace->Action = createInfo->action; // TODO: should i have pointer to actual FOpenXREmulatedAction?
	CreatedActionSpace->SubactionPath = createInfo->subactionPath;

	{
		FWriteScopeLock SessionLock(SessionMutex);
		CurrentSession->ActiveSpaces.Add(MoveTemp(CreatedActionSpace));
		*space = reinterpret_cast<XrSpace>(CurrentSession->ActiveSpaces.Last().Get());
	}

	{
		FOpenXREmulatedAction* OwningAction = reinterpret_cast<FOpenXREmulatedAction*>(createInfo->action);
		ActionPoseManager.RegisterEmulatedActionSpace(OwningAction->ActionName, *createInfo, *space);
	}

	return XrResult::XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerLocateSpace(XrSpace space, XrSpace baseSpace, XrTime time, XrSpaceLocation* location)
{
	if (!SpaceHandleCheck(space))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if (!SpaceHandleCheck(baseSpace))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	// TODO: time validation

	if ((location == nullptr) ||
		(location->type != XR_TYPE_SPACE_LOCATION))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	*location = ActionPoseManager.GetEmulatedPoseForTime(space, baseSpace, time);

	return XrResult::XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerDestroySpace(XrSpace space)
{
	if (!SpaceHandleCheck(space))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	const uint32 IndexToRemove = CurrentSession->ActiveSpaces.IndexOfByPredicate([space](const TUniquePtr<FOpenXREmulatedSpace>& EmulatedSpace)
	{
		return space == reinterpret_cast<XrSpace>(EmulatedSpace.Get());
	});
	CurrentSession->ActiveSpaces.RemoveAtSwap(IndexToRemove);

	return XrResult::XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerEnumerateViewConfigurations(XrInstance instance, XrSystemId systemId, uint32_t viewConfigurationTypeCapacityInput, uint32_t* viewConfigurationTypeCountOutput, XrViewConfigurationType* viewConfigurationTypes)
{
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if (!SystemIdCheck(systemId))
	{
		return XR_ERROR_SYSTEM_INVALID;
	}

	if (viewConfigurationTypeCountOutput == nullptr)
	{
		// log error
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (viewConfigurationTypeCapacityInput == 0)
	{
		*viewConfigurationTypeCountOutput = CaptureDecoder.GetViewConfigurationTypes().Num();
	}
	else
	{
		if (viewConfigurationTypes == nullptr)
		{
			// log error
			return XR_ERROR_VALIDATION_FAILURE;
		}

		const uint32 NumTypesToCopy = FMath::Min(viewConfigurationTypeCapacityInput, (uint32)CaptureDecoder.GetViewConfigurationTypes().Num());
		*viewConfigurationTypeCountOutput = NumTypesToCopy;

		FMemory::Memcpy(viewConfigurationTypes, CaptureDecoder.GetViewConfigurationTypes().GetData(), NumTypesToCopy * sizeof(XrViewConfigurationType));

	}

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerGetViewConfigurationProperties(XrInstance instance, XrSystemId systemId, XrViewConfigurationType viewConfigurationType, XrViewConfigurationProperties* configurationProperties)
{
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if (!SystemIdCheck(systemId))
	{
		return XR_ERROR_SYSTEM_INVALID;
	}

	if (!CaptureDecoder.GetViewConfigurationTypes().Contains(viewConfigurationType))
	{
		return XR_ERROR_VIEW_CONFIGURATION_TYPE_UNSUPPORTED;
	}

	if (!CaptureDecoder.GetViewConfigurationProperties().Contains(viewConfigurationType))
	{
		// TODO: We didn't capture the properties at capture time :/
		// Would be good to just automatically query all supported types on the capture side
		return XR_ERROR_RUNTIME_FAILURE;
	}

	if ((configurationProperties == nullptr) ||
		(configurationProperties->type != XR_TYPE_VIEW_CONFIGURATION_PROPERTIES))
	{
		// log error
		return XR_ERROR_VALIDATION_FAILURE;
	}

	*configurationProperties = CaptureDecoder.GetViewConfigurationProperties()[viewConfigurationType];

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerEnumerateViewConfigurationViews(XrInstance instance, XrSystemId systemId, XrViewConfigurationType viewConfigurationType, uint32_t viewCapacityInput, uint32_t* viewCountOutput, XrViewConfigurationView* views)
{
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if (!SystemIdCheck(systemId))
	{
		return XR_ERROR_SYSTEM_INVALID;
	}

	if (!CaptureDecoder.GetViewConfigurationTypes().Contains(viewConfigurationType))
	{
		return XR_ERROR_VIEW_CONFIGURATION_TYPE_UNSUPPORTED;
	}

	if (!CaptureDecoder.GetViewConfigurationViews().Contains(viewConfigurationType))
	{
		// TODO: We didn't capture the views at capture time :/
		// Would be good to just automatically query all supported types on the capture side
		return XR_ERROR_RUNTIME_FAILURE;
	}

	if (viewCountOutput == nullptr)
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	const TArray<XrViewConfigurationView>& CapturedViews = CaptureDecoder.GetViewConfigurationViews()[viewConfigurationType];

	if (viewCapacityInput == 0)
	{
		*viewCountOutput = CapturedViews.Num();
	}
	else
	{
	if (views == nullptr)
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	const uint32 NumViewsToCopy = FMath::Min(viewCapacityInput, (uint32)CapturedViews.Num());
	*viewCountOutput = NumViewsToCopy;

	FMemory::Memcpy(views, CapturedViews.GetData(), NumViewsToCopy * sizeof(XrViewConfigurationView));

	}

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerEnumerateSwapchainFormats(XrSession session, uint32_t formatCapacityInput, uint32_t* formatCountOutput, int64_t* formats)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if (formatCountOutput == nullptr)
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	// TODO: We don't actually have to return the REAL runtime formats
	// we can just return what we support here in emulation!

	if (formatCapacityInput == 0)
	{
		*formatCountOutput = CaptureDecoder.GetSwapchainFormats().Num();
	}
	else
	{
		if (formats == nullptr)
		{
			return XR_ERROR_VALIDATION_FAILURE;
		}

		const uint32 NumFormatsToCopy = FMath::Min(formatCapacityInput, (uint32)CaptureDecoder.GetSwapchainFormats().Num());
		*formatCountOutput = NumFormatsToCopy;

		FMemory::Memcpy(formats, CaptureDecoder.GetSwapchainFormats().GetData(), NumFormatsToCopy * sizeof(int64_t));
	}

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerCreateSwapchain(XrSession session, const XrSwapchainCreateInfo* createInfo, XrSwapchain* swapchain)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((createInfo == nullptr) ||
		(createInfo->type != XR_TYPE_SWAPCHAIN_CREATE_INFO) ||
		(swapchain == nullptr))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	{
		FReadScopeLock SessionLock(SessionMutex);
		if (!CurrentSession->bGraphicsBindingConfigured)
		{
			return XR_ERROR_VALIDATION_FAILURE;
		}
	}

	if ((createInfo->sampleCount > 1) ||
		(createInfo->faceCount > 1) ||
		(createInfo->mipCount > 1))
	{
		// TODO: do we want to support any of these swapchain variants??
		return XR_ERROR_RUNTIME_FAILURE;
	}

	FRHICommandListImmediate& RHICmdList = FRHICommandListImmediate::Get();

	TUniquePtr<FOpenXREmulatedSwapchain> NewSwapchain = MakeUnique<FOpenXREmulatedSwapchain>();

	static uint32 SwapchainCounter = 0;
	const uint32 SwapchainNum = SwapchainCounter++;

	NewSwapchain->DebugName = FString::Printf(TEXT("EmulatedSwapchain%d"), SwapchainNum);

	FRHITextureCreateDesc SwapchainImageDesc = createInfo->arraySize > 1 ?
		FRHITextureCreateDesc::Create2DArray(*NewSwapchain->DebugName).SetArraySize(createInfo->arraySize) :
		FRHITextureCreateDesc::Create2D(*NewSwapchain->DebugName);

	SwapchainImageDesc.SetExtent(createInfo->width, createInfo->height);
	SwapchainImageDesc.SetFormat(ConvertPlatformFormat(createInfo->format));
	//SwapchainImageDesc.SetInitialState(); // TODO: do we need an initial state??
	SwapchainImageDesc.SetFlags(ConvertXrSwapchainFlagsToTextureCreateFlags(createInfo));

	for (uint32 SwapchainImageIndex = 0; SwapchainImageIndex < kEmulatedSwapchainLength; SwapchainImageIndex++)
	{
		// TODO: handle alloc failure?!
		FTextureRHIRef NewImage = RHICmdList.CreateTexture(SwapchainImageDesc);
		NewSwapchain->TextureList[SwapchainImageIndex] = NewImage;
	}

	{
		*swapchain = reinterpret_cast<XrSwapchain>(NewSwapchain.Get());
		FWriteScopeLock SessionLock(SessionMutex);
		CurrentSession->ActiveSwapchains.Add(MoveTemp(NewSwapchain));
	}

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerDestroySwapchain(XrSwapchain swapchain)
{
	if (!SwapchainHandleCheck(swapchain))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	const uint32 IndexToRemove = CurrentSession->ActiveSwapchains.IndexOfByPredicate([swapchain](const TUniquePtr<FOpenXREmulatedSwapchain>& EmulatedSwapchain)
	{
		return swapchain == reinterpret_cast<XrSwapchain>(EmulatedSwapchain.Get());
	});
	CurrentSession->ActiveSwapchains.RemoveAtSwap(IndexToRemove);

	return XrResult::XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerEnumerateSwapchainImages(XrSwapchain swapchain, uint32_t imageCapacityInput, uint32_t* imageCountOutput, XrSwapchainImageBaseHeader* images)
{
	if (!SwapchainHandleCheck(swapchain))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if (imageCountOutput == nullptr)
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	const FOpenXREmulatedSwapchain* EmulatedSwapchain = reinterpret_cast<FOpenXREmulatedSwapchain*>(swapchain);

	if (imageCapacityInput == 0)
	{
		*imageCountOutput = EmulatedSwapchain->TextureList.Num();
	}
	else
	{
		if (images == nullptr)
		{
			return XR_ERROR_VALIDATION_FAILURE;
		}

		const uint32 NumImagesToCopy = FMath::Min(imageCapacityInput, (uint32)EmulatedSwapchain->TextureList.Num());
		*imageCountOutput = NumImagesToCopy;

		XrStructureType BindingType = XR_TYPE_UNKNOWN;
		{
			FReadScopeLock SessionLock(SessionMutex);
			BindingType = CurrentSession->GraphicsBinding.Base.type;
		}

#if defined(XR_USE_GRAPHICS_API_D3D12)
		if ((images->type == XR_TYPE_SWAPCHAIN_IMAGE_D3D12_KHR) && (BindingType == XR_TYPE_GRAPHICS_BINDING_D3D12_KHR))
		{
			TArrayView<XrSwapchainImageD3D12KHR> OutImages((XrSwapchainImageD3D12KHR*)images, NumImagesToCopy);
			XrSwapchainImageD3D12KHR D3D12Image{};
			D3D12Image.type = XR_TYPE_SWAPCHAIN_IMAGE_D3D12_KHR;

			for (uint32 ImageIndex = 0; ImageIndex < NumImagesToCopy; ImageIndex++)
			{
				D3D12Image.texture = (ID3D12Resource*)EmulatedSwapchain->TextureList[ImageIndex]->GetNativeResource();
				OutImages[ImageIndex] = D3D12Image;
			}
		}
		else
#endif // defined(XR_USE_GRAPHICS_API_D3D12)
		{
			return XR_ERROR_RUNTIME_FAILURE;
		}
	}

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerAcquireSwapchainImage(XrSwapchain swapchain, const XrSwapchainImageAcquireInfo* acquireInfo, uint32_t* index)
{
	if (!SwapchainHandleCheck(swapchain))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((acquireInfo != nullptr) &&
		(acquireInfo->type != XR_TYPE_SWAPCHAIN_IMAGE_ACQUIRE_INFO))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (index == nullptr)
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	FOpenXREmulatedSwapchain* EmulatedSwapchain = reinterpret_cast<FOpenXREmulatedSwapchain*>(swapchain);

	const uint32 AcquireIndex = EmulatedSwapchain->LabelCounter++ % kEmulatedSwapchainLength;

	if (EmulatedSwapchain->ImageStateList[AcquireIndex].APIState != FOpenXREmulatedSwapchain::ESwapchainImageAPIState::Released)
	{
		return XR_ERROR_CALL_ORDER_INVALID;
	}

	EmulatedSwapchain->ImageStateList[AcquireIndex].APIState = FOpenXREmulatedSwapchain::ESwapchainImageAPIState::Acquired;
	EmulatedSwapchain->ImageStateList[AcquireIndex].AcquireTimeCycles = FPlatformTime::Cycles();
	*index = AcquireIndex;

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerWaitSwapchainImage(XrSwapchain swapchain, const XrSwapchainImageWaitInfo* waitInfo)
{
	if (!SwapchainHandleCheck(swapchain))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((waitInfo != nullptr) &&
		(waitInfo->type != XR_TYPE_SWAPCHAIN_IMAGE_WAIT_INFO))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	FOpenXREmulatedSwapchain* EmulatedSwapchain = reinterpret_cast<FOpenXREmulatedSwapchain*>(swapchain);

	// scan for oldest acquire
	int32 WaitIndex = INDEX_NONE;
	int64 OldestAcquireTime = INT64_MAX;

	// TODO: is there a predicate for this operation?
	for (uint32 ScanIndex = 0; ScanIndex < kEmulatedSwapchainLength; ScanIndex++)
	{
		if (EmulatedSwapchain->ImageStateList[ScanIndex].APIState == FOpenXREmulatedSwapchain::ESwapchainImageAPIState::Acquired &&
			EmulatedSwapchain->ImageStateList[ScanIndex].AcquireTimeCycles < OldestAcquireTime)
		{
			WaitIndex = ScanIndex;
			OldestAcquireTime = EmulatedSwapchain->ImageStateList[ScanIndex].AcquireTimeCycles;
		}
	}

	if (WaitIndex == INDEX_NONE)
	{
		return XR_ERROR_CALL_ORDER_INVALID;
	}

	EmulatedSwapchain->ImageStateList[WaitIndex].APIState = FOpenXREmulatedSwapchain::ESwapchainImageAPIState::Waited;

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerReleaseSwapchainImage(XrSwapchain swapchain, const XrSwapchainImageReleaseInfo* releaseInfo) 
{
	if (!SwapchainHandleCheck(swapchain))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((releaseInfo != nullptr) &&
		(releaseInfo->type != XR_TYPE_SWAPCHAIN_IMAGE_RELEASE_INFO))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	FOpenXREmulatedSwapchain* EmulatedSwapchain = reinterpret_cast<FOpenXREmulatedSwapchain*>(swapchain);

	// scan for oldest acquire that is also waited on
	uint32 ReleaseIndex = INDEX_NONE;
	int64 OldestAcquireTime = INT64_MAX;

	// TODO: is there a predicate for this operation?
	for (uint32 ScanIndex = 0; ScanIndex < kEmulatedSwapchainLength; ScanIndex++)
	{
		if (EmulatedSwapchain->ImageStateList[ScanIndex].APIState == FOpenXREmulatedSwapchain::ESwapchainImageAPIState::Waited &&
			EmulatedSwapchain->ImageStateList[ScanIndex].AcquireTimeCycles < OldestAcquireTime)
		{
			ReleaseIndex = ScanIndex;
			OldestAcquireTime = EmulatedSwapchain->ImageStateList[ScanIndex].AcquireTimeCycles;
		}
	}

	if (ReleaseIndex == INDEX_NONE)
	{
		return XR_ERROR_CALL_ORDER_INVALID;
	}

	EmulatedSwapchain->ImageStateList[ReleaseIndex].APIState = FOpenXREmulatedSwapchain::ESwapchainImageAPIState::Released;

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerBeginSession(XrSession session, const XrSessionBeginInfo* beginInfo)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((beginInfo == nullptr) ||
		(beginInfo->type != XR_TYPE_SESSION_BEGIN_INFO) ||
		(!CaptureDecoder.GetViewConfigurationTypes().Contains(beginInfo->primaryViewConfigurationType)))
	{
		// log error
		return XR_ERROR_VALIDATION_FAILURE;
	}

	{
		FReadScopeLock SessionLock(SessionMutex);
		if (CurrentSession->CurrentSessionState != XR_SESSION_STATE_READY)
		{
			return XR_ERROR_SESSION_NOT_READY;
		}

		if (CurrentSession->bRunning)
		{
			return XR_ERROR_SESSION_RUNNING;
		}
	}

	{
		FWriteScopeLock SessionLock(SessionMutex);
		CurrentSession->ActiveViewConfigType = beginInfo->primaryViewConfigurationType;
		CurrentSession->bRunning = true;
	}

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerEndSession(XrSession session)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	{
		FReadScopeLock SessionLock(SessionMutex);
		if (CurrentSession->CurrentSessionState != XR_SESSION_STATE_STOPPING)
		{
			return XR_ERROR_SESSION_NOT_STOPPING;
		}

		if (!CurrentSession->bRunning)
		{
			return XR_ERROR_SESSION_NOT_RUNNING;
		}
	}

	{
		FWriteScopeLock SessionLock(SessionMutex);
		CurrentSession->bRunning = false;
		UpdateSessionState(*CurrentSession, XR_SESSION_STATE_IDLE);
	}
	PendingApplicationUpdateQueue.Enqueue(FOpenXRUpdateEvent(XR_SESSION_STATE_IDLE));

	// TODO: Reset any other session state?

	// TODO: Do I need any actual pause between Idle and Exiting?
	{
		FWriteScopeLock SessionLock(SessionMutex);
		if (CurrentSession->bExitRequested)
		{
			CurrentSession->bExitRequested = false;
			UpdateSessionState(*CurrentSession, XR_SESSION_STATE_EXITING);
			PendingApplicationUpdateQueue.Enqueue(FOpenXRUpdateEvent(XR_SESSION_STATE_EXITING));
		}
	}

	return XR_SUCCESS;

}

XrResult FOpenXREmulationLayer::XrLayerRequestExitSession(XrSession session)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if (!IsCurrentSessionRunning())
	{
		return XR_ERROR_SESSION_NOT_RUNNING;
	}

	{
		FWriteScopeLock SessionLock(SessionMutex);

		// TODO: Do we care about multiple exit requests?
		CurrentSession->bExitRequested = true;

		if (CurrentSession->CurrentSessionState == XR_SESSION_STATE_READY)
		{
			// OpenXR spec says we have to go through synchronized before going to stopping
			UpdateSessionState(*CurrentSession, XR_SESSION_STATE_SYNCHRONIZED);
			PendingApplicationUpdateQueue.Enqueue(FOpenXRUpdateEvent(XR_SESSION_STATE_SYNCHRONIZED));
		}
		if (CurrentSession->CurrentSessionState == XR_SESSION_STATE_FOCUSED)
		{
			UpdateSessionState(*CurrentSession, XR_SESSION_STATE_VISIBLE);
			PendingApplicationUpdateQueue.Enqueue(FOpenXRUpdateEvent(XR_SESSION_STATE_VISIBLE));
		}
		if (CurrentSession->CurrentSessionState == XR_SESSION_STATE_VISIBLE)
		{
			UpdateSessionState(*CurrentSession, XR_SESSION_STATE_SYNCHRONIZED);
			PendingApplicationUpdateQueue.Enqueue(FOpenXRUpdateEvent(XR_SESSION_STATE_SYNCHRONIZED));
		}
		if (CurrentSession->CurrentSessionState == XR_SESSION_STATE_SYNCHRONIZED)
		{
			UpdateSessionState(*CurrentSession, XR_SESSION_STATE_STOPPING);
			PendingApplicationUpdateQueue.Enqueue(FOpenXRUpdateEvent(XR_SESSION_STATE_STOPPING));
		}
	}

	return XrResult::XR_SUCCESS;
}

// TODO: Wait frame will kick off synchronization process. We could just enter it instantly
// We do need to be careful with how UE uses BeginFrame + EndFrame? The spec isn't clear about what
// they should return 'as we synchronize'. I guess they can just return before we enter synchronized state

XrResult FOpenXREmulationLayer::XrLayerWaitFrame(XrSession session, const XrFrameWaitInfo* frameWaitInfo, XrFrameState* frameState)
{ 
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((frameWaitInfo != nullptr) &&
		(frameWaitInfo->type != XR_TYPE_FRAME_WAIT_INFO))
	{
		// We support null XrFrameWaitInfo
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if ((frameState == nullptr) ||
		(frameState->type != XR_TYPE_FRAME_STATE))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (!IsCurrentSessionRunning())
	{
		return XR_ERROR_SESSION_NOT_RUNNING;
	}

	// transition session state from synchronized to visible to focused
	if (CurrentSession->CurrentSessionState == XR_SESSION_STATE_READY)
	{
		FWriteScopeLock SessionLock(SessionMutex);

		UpdateSessionState(*CurrentSession, XR_SESSION_STATE_SYNCHRONIZED);
		PendingApplicationUpdateQueue.Enqueue(FOpenXRUpdateEvent(XR_SESSION_STATE_SYNCHRONIZED));
		UpdateSessionState(*CurrentSession, XR_SESSION_STATE_VISIBLE);
		PendingApplicationUpdateQueue.Enqueue(FOpenXRUpdateEvent(XR_SESSION_STATE_VISIBLE));
		UpdateSessionState(*CurrentSession, XR_SESSION_STATE_FOCUSED);
		PendingApplicationUpdateQueue.Enqueue(FOpenXRUpdateEvent(XR_SESSION_STATE_FOCUSED));
	}

	// wait for beginframe event (first frame is already triggered)
	CurrentSession->WaitFrameAdvanceEvent->Wait();

	// signal that wait frame was called on the session to be consumed by begin frame
	CurrentSession->WaitFrameFrameCounterPipe.Enqueue(++CurrentSession->WaitFrameCounter);

	// framestate
	frameState->predictedDisplayPeriod = 8300000; // 120 fps
	frameState->shouldRender = XR_TRUE;
	// TODO: i'm just making up 4 frames of latency
	frameState->predictedDisplayTime = UETicksToXrTime(FDateTime::Now().GetTicks()) + (4 * frameState->predictedDisplayPeriod);

	ActionPoseManager.AddEmulatedFrameTime(frameState->predictedDisplayTime, CurrentSession->WaitFrameCounter);

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerBeginFrame(XrSession session, const XrFrameBeginInfo* frameBeginInfo)
{ 
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	// in the fail cases below, we do trigger the WaitFrameAdvanceEvent.
	// unclear if we want to 'hold' them if something strange is going on
	// TODO: log warnings/errors in these cases

	if ((frameBeginInfo != nullptr) &&
		(frameBeginInfo->type != XR_TYPE_FRAME_BEGIN_INFO))
	{
		CurrentSession->WaitFrameAdvanceEvent->Trigger();

		// We support null XrFrameBeginInfo
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (!IsCurrentSessionRunning())
	{
		CurrentSession->WaitFrameAdvanceEvent->Trigger();
		return XR_ERROR_SESSION_NOT_RUNNING;
	}

	// verify wait frame was called before
	if (CurrentSession->WaitFrameFrameCounterPipe.IsEmpty())
	{
		// no trigger because we don't want de-sync with WaitFrame
		return XR_ERROR_CALL_ORDER_INVALID;
	}

	if (!CurrentSession->WaitFrameFrameCounterPipe.Dequeue(CurrentSession->BeginFrameCounter))
	{
		// uh, this is really bad if we don't get something from the pipe
		checkf(0, TEXT("Failed to get frame counter from WaitFrame"));
		return XR_ERROR_RUNTIME_FAILURE;
	}

	// release wait frame event
	CurrentSession->WaitFrameAdvanceEvent->Trigger();

	// We don't need to check FrameSubmitState with synchronization because the 
	// application is OBLIGED to externally sync access. Still, we could protect with
	// a mutex to be extra safe
	const XrResult BeginFrameResult = (CurrentSession->FrameSubmitState == EFrameSubmissionState::FrameReadyToFinish) ? XR_FRAME_DISCARDED : XR_SUCCESS;

	CurrentSession->FrameSubmitState = EFrameSubmissionState::FrameReadyToFinish;

	return BeginFrameResult;
}

XrResult FOpenXREmulationLayer::XrLayerEndFrame(XrSession session, const XrFrameEndInfo* frameEndInfo)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((frameEndInfo == nullptr) ||
		(frameEndInfo->type != XR_TYPE_FRAME_END_INFO))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	// TODO: validate display time, if possible?

	if (!CaptureDecoder.GetEnvironmentBlendModes().Contains(frameEndInfo->environmentBlendMode))
	{
		return XR_ERROR_ENVIRONMENT_BLEND_MODE_UNSUPPORTED;
	}

	if (CurrentSession->FrameSubmitState != EFrameSubmissionState::FrameReadyToFinish)
	{
		return XR_ERROR_CALL_ORDER_INVALID;
	}
	CurrentSession->FrameSubmitState = EFrameSubmissionState::FrameReadyToBegin;

	// TODO: support any layers :p

	//if (frameEndInfo->layerCount > 0)
	//{
	//	UE_LOGF(LogXRScribeEmulate, Error, "EndFrame: unsupported layer count");
	//	return XR_ERROR_LAYER_LIMIT_EXCEEDED;
	//}

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerLocateViews(XrSession session, const XrViewLocateInfo* viewLocateInfo, XrViewState* viewState, uint32_t viewCapacityInput, uint32_t* viewCountOutput, XrView* views)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((viewLocateInfo == nullptr) ||
		(viewLocateInfo->type != XR_TYPE_VIEW_LOCATE_INFO))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if ((viewState == nullptr) ||
		(viewState->type != XR_TYPE_VIEW_STATE))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (viewCountOutput == nullptr)
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (viewLocateInfo->displayTime == 0)
	{
		// TODO: actually validate the time
		return XR_ERROR_TIME_INVALID;
	}

	if (!CaptureDecoder.GetViewConfigurationTypes().Contains(viewLocateInfo->viewConfigurationType))
	{
		return XR_ERROR_VIEW_CONFIGURATION_TYPE_UNSUPPORTED;
	}

	if (!SpaceHandleCheck(viewLocateInfo->space))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	// we only support locating views relative to HMD (for now). This means the views are actually
	// relatively static unless user would change their IPD. Might change with gaze tracking
	// TODO: support arbitrary spaces for view location (or maybe constrain to reference spaces)
	{
		const FOpenXREmulatedSpace* EmulatedSpace = reinterpret_cast<FOpenXREmulatedSpace*>(viewLocateInfo->space);
		if ((EmulatedSpace->SpaceType != FOpenXREmulatedSpace::ESpaceType::Reference) ||
			(EmulatedSpace->RefSpaceType != XR_REFERENCE_SPACE_TYPE_VIEW))
		{
			return XR_ERROR_RUNTIME_FAILURE;
		}
	}

	const TMap<XrViewConfigurationType, TArray<FOpenXRLocateViewsPacket>>& ViewLocations = CaptureDecoder.GetViewLocations();

	if (!ViewLocations.Contains(viewLocateInfo->viewConfigurationType))
	{
		return XR_ERROR_RUNTIME_FAILURE;
	}

	const TArray<FOpenXRLocateViewsPacket>& ViewLocationHistoryForType = ViewLocations[viewLocateInfo->viewConfigurationType];

	// TODO: fetch history, maybe? We could just set the bits here
	viewState->viewStateFlags = XR_VIEW_STATE_ORIENTATION_VALID_BIT | XR_VIEW_STATE_POSITION_VALID_BIT | XR_VIEW_STATE_ORIENTATION_TRACKED_BIT | XR_VIEW_STATE_POSITION_TRACKED_BIT;

	if (viewCapacityInput == 0)
	{
		check(ViewLocationHistoryForType.Num() > 0);
		*viewCountOutput = ViewLocationHistoryForType[0].Views.Num();
	}
	else
	{
		if (viewCapacityInput < (uint32)ViewLocationHistoryForType[0].Views.Num())
		{
			return XR_ERROR_SIZE_INSUFFICIENT;
		}

		if (views == nullptr)
		{
			return XR_ERROR_VALIDATION_FAILURE;
		}

		const uint32 NumViewsToCopy = FMath::Min(viewCapacityInput, (uint32)ViewLocationHistoryForType[0].Views.Num());
		*viewCountOutput = NumViewsToCopy;

		FMemory::Memcpy(views, ViewLocationHistoryForType[0].Views.GetData(), NumViewsToCopy * sizeof(XrView));

		// TODO: actually fetch history of view locations
	}

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerStringToPath(XrInstance instance, const char* pathString, XrPath* path)
{
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((pathString == nullptr) ||
		(path == nullptr))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	XrPath* FoundPath = nullptr;
	{
		FReadScopeLock InstanceLock(InstanceMutex);
		FoundPath = CurrentInstance->ANSIStringToPathMap.Find(pathString);
	}

	if (FoundPath != nullptr)
	{
		*path = *FoundPath;
	}
	else
	{
		FOpenXREmulatedPath EmulatedPathString;
		const int32 PathLen = FCStringAnsi::Strlen(pathString);
		if (PathLen > (XR_MAX_PATH_LENGTH - 1))
		{
			return XR_ERROR_PATH_COUNT_EXCEEDED;
		}

		// TODO: well-formed path string filtering

		FCStringAnsi::Strncpy(EmulatedPathString.GetData(), pathString, XR_MAX_PATH_LENGTH);
		{
			FWriteScopeLock InstanceLock(InstanceMutex);
			const int32 AddedIndex = CurrentInstance->PathList.Add(MoveTemp(EmulatedPathString));
			const XrPath AddedPath = AddedIndex;
			CurrentInstance->ANSIStringToPathMap.Add(pathString, AddedPath);
			ActionPoseManager.RegisterEmulatedPath(FName(ANSI_TO_TCHAR(pathString)), AddedPath);
			*path = AddedPath;
		}
	}

	return XR_SUCCESS;
}
XrResult FOpenXREmulationLayer::XrLayerPathToString(XrInstance instance, XrPath path, uint32_t bufferCapacityInput, uint32_t* bufferCountOutput, char* buffer)
{
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if (path == XR_NULL_PATH)
	{
		return XR_ERROR_PATH_INVALID;
	}

	const int32 PathIndex = path;
	{
		FReadScopeLock InstanceLock(InstanceMutex);
		if (PathIndex > CurrentInstance->PathList.Num())
		{
			return XR_ERROR_PATH_INVALID;
		}
	}

	if (bufferCountOutput == nullptr)
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	int32 PathLen = 0;
	{
		FReadScopeLock InstanceLock(InstanceMutex);
		PathLen = FCStringAnsi::Strlen(CurrentInstance->PathList[PathIndex].GetData()) + 1;
	}

	*bufferCountOutput = PathLen;

	if (bufferCapacityInput > 0)
	{
		if (buffer == nullptr)
		{
			return XR_ERROR_VALIDATION_FAILURE;
		}

		if (bufferCapacityInput < (uint32)PathLen)
		{
			return XR_ERROR_SIZE_INSUFFICIENT;
		}

		{
			FReadScopeLock InstanceLock(InstanceMutex);
			FCStringAnsi::Strncpy(buffer, CurrentInstance->PathList[PathIndex].GetData(), PathLen);
		}
	}

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerCreateActionSet(XrInstance instance, const XrActionSetCreateInfo* createInfo, XrActionSet* actionSet)
{
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((createInfo == nullptr) ||
		(createInfo->type != XR_TYPE_ACTION_SET_CREATE_INFO) ||
		(actionSet == nullptr))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (createInfo->actionSetName[0] == '\0')
	{
		return XR_ERROR_NAME_INVALID;
	}

	if (createInfo->localizedActionSetName[0] == '\0')
	{
		return XR_ERROR_LOCALIZED_NAME_INVALID;
	}

	{
		FReadScopeLock InstanceLock(InstanceMutex);
		if (CurrentInstance->ActiveActionSetNames.Contains(&createInfo->actionSetName[0]))
		{
			return XR_ERROR_NAME_DUPLICATED;
		}
	}

	// TODO: validate names

	TUniquePtr<FOpenXREmulatedActionSet> ActionSet = MakeUnique<FOpenXREmulatedActionSet>();
	ActionSet->Priority = createInfo->priority;
	FCStringAnsi::Strncpy(ActionSet->ActionSetName.GetData(), createInfo->actionSetName, XR_MAX_ACTION_NAME_SIZE);
	FCStringAnsi::Strncpy(ActionSet->LocalizedActionSetName.GetData(), createInfo->localizedActionSetName, XR_MAX_LOCALIZED_ACTION_NAME_SIZE);

	{
		FWriteScopeLock InstanceLock(InstanceMutex);
		CurrentInstance->ActiveActionSetNames.Add(ActionSet->ActionSetName.GetData());
		CurrentInstance->ActiveActionSets.Add(MoveTemp(ActionSet));
		*actionSet = reinterpret_cast<XrActionSet>(CurrentInstance->ActiveActionSets.Last().Get());
	}

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerDestroyActionSet(XrActionSet actionSet)
{
	if (!ActionSetHandleCheck(actionSet))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	FOpenXREmulatedActionSet* EmulatedActionSet = reinterpret_cast<FOpenXREmulatedActionSet*>(actionSet);
	for (const TUniquePtr<FOpenXREmulatedAction>& EmulatedAction : EmulatedActionSet->Actions)
	{
		CurrentInstance->AllActiveActions.RemoveSwap(reinterpret_cast<XrAction>(EmulatedAction.Get()));
	}

	const uint32 IndexToRemove = CurrentInstance->ActiveActionSets.IndexOfByPredicate([actionSet](const TUniquePtr<FOpenXREmulatedActionSet>& EmulatedActionSet)
	{
		return actionSet == reinterpret_cast<XrActionSet>(EmulatedActionSet.Get());
	});
	CurrentInstance->ActiveActionSets.RemoveAtSwap(IndexToRemove);

	return XrResult::XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerCreateAction(XrActionSet actionSet, const XrActionCreateInfo* createInfo, XrAction* action) 
{
	if (!ActionSetHandleCheck(actionSet))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((createInfo == nullptr) ||
		(createInfo->type != XR_TYPE_ACTION_CREATE_INFO) ||
		(action == nullptr))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (createInfo->actionName[0] == '\0')
	{
		return XR_ERROR_NAME_INVALID;
	}

	if (createInfo->localizedActionName[0] == '\0')
	{
		return XR_ERROR_LOCALIZED_NAME_INVALID;
	}

	// TODO: check for duplicate action names
	// TODO: check name validity

	if ((createInfo->actionType != XR_ACTION_TYPE_BOOLEAN_INPUT) &&
		(createInfo->actionType != XR_ACTION_TYPE_FLOAT_INPUT) &&
		(createInfo->actionType != XR_ACTION_TYPE_VECTOR2F_INPUT) &&
		(createInfo->actionType != XR_ACTION_TYPE_POSE_INPUT) &&
		(createInfo->actionType != XR_ACTION_TYPE_VIBRATION_OUTPUT))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if ((createInfo->countSubactionPaths != 0) && (createInfo->subactionPaths == nullptr))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	FOpenXREmulatedActionSet* EmulatedActionSet = reinterpret_cast<FOpenXREmulatedActionSet*>(actionSet);

	if (EmulatedActionSet->bAttached)
	{
		return XR_ERROR_ACTIONSETS_ALREADY_ATTACHED;
	}

	{
		FReadScopeLock InstanceLock(InstanceMutex);
		if (EmulatedActionSet->ActionNames.Contains(&createInfo->actionName[0]))
		{
			return XR_ERROR_NAME_DUPLICATED;
		}
	}

	TUniquePtr<FOpenXREmulatedAction> Action = MakeUnique<FOpenXREmulatedAction>();
	
	Action->OwningActionSet = EmulatedActionSet;
	Action->ActionType = createInfo->actionType;
	FCStringAnsi::Strncpy(Action->ActionName.GetData(), createInfo->actionName, XR_MAX_ACTION_NAME_SIZE);
	FCStringAnsi::Strncpy(Action->LocalizedActionName.GetData(), createInfo->localizedActionName, XR_MAX_LOCALIZED_ACTION_NAME_SIZE);
	Action->SubactionPaths.Append(createInfo->subactionPaths, createInfo->countSubactionPaths);

	*action = reinterpret_cast<XrAction>(Action.Get());

	{
		FWriteScopeLock InstanceLock(InstanceMutex);
		EmulatedActionSet->ActionNames.Add(Action->ActionName.GetData());
		EmulatedActionSet->Actions.Add(MoveTemp(Action));
		CurrentInstance->AllActiveActions.Add(*action);

		ActionPoseManager.RegisterEmulatedAction(FName(ANSI_TO_TCHAR(createInfo->actionName)), *action, createInfo->actionType);
	}

	return XrResult::XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerDestroyAction(XrAction action)
{
	if (!ActionHandleCheck(action))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	FOpenXREmulatedActionSet* EmulatedActionSet = reinterpret_cast<FOpenXREmulatedAction*>(action)->OwningActionSet;
	const uint32 IndexToRemove = EmulatedActionSet->Actions.IndexOfByPredicate([action](const TUniquePtr<FOpenXREmulatedAction>& EmulatedAction)
	{
		return action == reinterpret_cast<XrAction>(EmulatedAction.Get());
	});
	EmulatedActionSet->Actions.RemoveAtSwap(IndexToRemove);

	{
		FWriteScopeLock InstanceLock(InstanceMutex);
		CurrentInstance->AllActiveActions.RemoveSwap(action);
	}

	// TODO: make common function to remove between DestroyActionSet and DestroyActions
	return XrResult::XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerSuggestInteractionProfileBindings(XrInstance instance, const XrInteractionProfileSuggestedBinding* suggestedBindings)
{
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((suggestedBindings == nullptr) ||
		(suggestedBindings->type != XR_TYPE_INTERACTION_PROFILE_SUGGESTED_BINDING))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	// TODO: do something with this info??
	// probably pick an interaction profile that matches the captured runtime/device

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerAttachSessionActionSets(XrSession session, const XrSessionActionSetsAttachInfo* attachInfo)
{ 
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((attachInfo == nullptr) ||
		(attachInfo->type != XR_TYPE_SESSION_ACTION_SETS_ATTACH_INFO) ||
		(attachInfo->countActionSets == 0) ||
		(attachInfo->actionSets == nullptr))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	{
		FReadScopeLock SessionLock(SessionMutex);
		if (CurrentSession->bActionSetsAttached)
		{
			return XR_ERROR_ACTIONSETS_ALREADY_ATTACHED;
		}
	}

	// TODO: save off list of action sets + sets in session

	TSet<XrActionSet> AttachedActionSets;
	TSet<XrAction> AttachedActions;

	for (uint32 ActionSetIndex = 0; ActionSetIndex < attachInfo->countActionSets; ActionSetIndex++)
	{
		if (!ActionSetHandleCheck(attachInfo->actionSets[ActionSetIndex]))
		{
			return XR_ERROR_HANDLE_INVALID;
		}

		AttachedActionSets.Add(attachInfo->actionSets[ActionSetIndex]);

		FOpenXREmulatedActionSet* EmulatedActionSet = reinterpret_cast<FOpenXREmulatedActionSet*>(attachInfo->actionSets[ActionSetIndex]);
		check(!EmulatedActionSet->bAttached);
		EmulatedActionSet->bAttached = true;

		for (const TUniquePtr<FOpenXREmulatedAction>& Action : EmulatedActionSet->Actions)
		{
			AttachedActions.Add(reinterpret_cast<XrAction>(Action.Get()));
		}
	}

	{
		FWriteScopeLock SessionLock(SessionMutex);
		CurrentSession->bActionSetsAttached = true;
		CurrentSession->AttachedActionSets = MoveTemp(AttachedActionSets);
		CurrentSession->AttachedActions = MoveTemp(AttachedActions);
	}

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerGetCurrentInteractionProfile(XrSession session, XrPath topLevelUserPath, XrInteractionProfileState* interactionProfile)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if (topLevelUserPath == XR_NULL_PATH)
	{
		return XR_ERROR_PATH_INVALID;
	}

	if ((interactionProfile == nullptr) ||
		(interactionProfile->type != XR_TYPE_INTERACTION_PROFILE_STATE))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	// TODO: return the interaction profile that is actually bound
	interactionProfile->interactionProfile = XR_NULL_PATH;

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerGetActionStateBoolean(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStateBoolean* state)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((getInfo == nullptr) ||
		(getInfo->type != XR_TYPE_ACTION_STATE_GET_INFO))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if ((state == nullptr) ||
		(state->type != XR_TYPE_ACTION_STATE_BOOLEAN))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	// TODO: check action exists and is attached
	// TODO: check subaction path

	*state = ActionPoseManager.GetEmulatedActionStateBoolean(getInfo, UETicksToXrTime(LastSyncTimeTicks));

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerGetActionStateFloat(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStateFloat* state)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((getInfo == nullptr) ||
		(getInfo->type != XR_TYPE_ACTION_STATE_GET_INFO))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if ((state == nullptr) ||
		(state->type != XR_TYPE_ACTION_STATE_FLOAT))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	// TODO: check action exists and is attached
	// TODO: check subaction path

	*state = ActionPoseManager.GetEmulatedActionStateFloat(getInfo, UETicksToXrTime(LastSyncTimeTicks));

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerGetActionStateVector2f(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStateVector2f* state)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((getInfo == nullptr) ||
		(getInfo->type != XR_TYPE_ACTION_STATE_GET_INFO))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if ((state == nullptr) ||
		(state->type != XR_TYPE_ACTION_STATE_VECTOR2F))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	// TODO: check action exists and is attached
	// TODO: check subaction path

	*state = ActionPoseManager.GetEmulatedActionStateVector2f(getInfo, UETicksToXrTime(LastSyncTimeTicks));


	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerGetActionStatePose(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStatePose* state)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((getInfo == nullptr) ||
		(getInfo->type != XR_TYPE_ACTION_STATE_GET_INFO))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if ((state == nullptr) ||
		(state->type != XR_TYPE_ACTION_STATE_POSE))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (!ActionHandleCheck(getInfo->action))
	{
		UE_LOGF(LogXRScribeEmulate, Error, "GetActionStatePose: unknown action");
		return XR_ERROR_VALIDATION_FAILURE;
	}

	FOpenXREmulatedAction* Action = reinterpret_cast<FOpenXREmulatedAction*>(getInfo->action);
	if (!Action->OwningActionSet->bAttached)
	{
		return XR_ERROR_ACTIONSET_NOT_ATTACHED;
	}

	if (Action->ActionType != XR_ACTION_TYPE_POSE_INPUT)
	{
		return XR_ERROR_ACTION_TYPE_MISMATCH;
	}

	// TODO: check subaction path?

	state->isActive = ActionPoseManager.DoesActionContainPoseHistory(Action->ActionName);

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerSyncActions(XrSession session, const	XrActionsSyncInfo* syncInfo)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if ((syncInfo == nullptr) ||
		(syncInfo->type != XR_TYPE_ACTIONS_SYNC_INFO))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	{
		FReadScopeLock SessionLock(SessionMutex);
		if (CurrentSession->CurrentSessionState != XR_SESSION_STATE_FOCUSED)
		{
			return XR_SESSION_NOT_FOCUSED;
		}
	}

	TSet<XrActionSet> AttachedActionSets;
	{
		FReadScopeLock SessionLock(SessionMutex);
		AttachedActionSets = CurrentSession->AttachedActionSets;
	}
	for (uint32 ActionSetIndex = 0; ActionSetIndex < syncInfo->countActiveActionSets; ActionSetIndex++)
	{
		if (!AttachedActionSets.Contains(syncInfo->activeActionSets[ActionSetIndex].actionSet))
		{
			return XR_ERROR_ACTIONSET_NOT_ATTACHED;
		}
	}


	// TODO: validate the action set subaction path

	//LastSyncTimeTicks = FDateTime::Now().GetTicks() - StartTimeTicks;
	LastSyncTimeTicks = FDateTime::Now().GetTicks();

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerEnumerateBoundSourcesForAction(XrSession session, const XrBoundSourcesForActionEnumerateInfo* enumerateInfo, uint32_t sourceCapacityInput, uint32_t* sourceCountOutput, XrPath* sources) 
{
	return UnsupportedFuncPlaceholder(EOpenXRAPIPacketId::EnumerateBoundSourcesForAction);
}
XrResult FOpenXREmulationLayer::XrLayerGetInputSourceLocalizedName(XrSession session, const XrInputSourceLocalizedNameGetInfo* getInfo, uint32_t bufferCapacityInput, uint32_t* bufferCountOutput, char* buffer)
{
	return UnsupportedFuncPlaceholder(EOpenXRAPIPacketId::GetInputSourceLocalizedName);
}

// TODO: do something with validating haptic feedback?
XrResult FOpenXREmulationLayer::XrLayerApplyHapticFeedback(XrSession session, const XrHapticActionInfo* hapticActionInfo, const XrHapticBaseHeader* hapticFeedback)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	return XR_SUCCESS;
}

XrResult FOpenXREmulationLayer::XrLayerStopHapticFeedback(XrSession session, const XrHapticActionInfo* hapticActionInfo)
{
	if (!SessionHandleCheck(session))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	return XR_SUCCESS;
}

// Global extensions

// XR_KHR_loader_init
XrResult FOpenXREmulationLayer::XrLayerInitializeLoaderKHR(const XrLoaderInitInfoBaseHeaderKHR* loaderInitInfo)
{
	// TODO: We won't actually capture this because of it's used in FOpenXRHMDModule::GetDefaultLoader
	// We need to move that code out of GetDefaultLoader and have it execute later in InitInstance.
	return UnsupportedFuncPlaceholder(EOpenXRAPIPacketId::InitializeLoaderKHR);
}

// Instance extensions

// XR_KHR_visibility_mask
XrResult FOpenXREmulationLayer::XrLayerGetVisibilityMaskKHR(XrSession session, XrViewConfigurationType viewConfigurationType, uint32_t viewIndex, XrVisibilityMaskTypeKHR visibilityMaskType, XrVisibilityMaskKHR* visibilityMask)
{
	// TODO: return the vis mask!
	return UnsupportedFuncPlaceholder(EOpenXRAPIPacketId::GetVisibilityMaskKHR);
}

#if defined(XR_USE_GRAPHICS_API_D3D11)
// XR_KHR_D3D11_enable
XrResult FOpenXREmulationLayer::XrLayerGetD3D11GraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsD3D11KHR* graphicsRequirements)
{
	return UnsupportedFuncPlaceholder(EOpenXRAPIPacketId::GetD3D11GraphicsRequirementsKHR);
}
#endif

#if defined(XR_USE_GRAPHICS_API_D3D12)
// XR_KHR_D3D12_enable
XrResult FOpenXREmulationLayer::XrLayerGetD3D12GraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsD3D12KHR* graphicsRequirements)
{
	if (!InstanceHandleCheck(instance))
	{
		return XR_ERROR_HANDLE_INVALID;
	}

	if (!SystemIdCheck(systemId))
	{
		return XR_ERROR_SYSTEM_INVALID;
	}

	if ((graphicsRequirements == nullptr) ||
		(graphicsRequirements->type != XR_TYPE_GRAPHICS_REQUIREMENTS_D3D12_KHR))
	{
		return XR_ERROR_VALIDATION_FAILURE;
	}

	if (GDynamicRHI != nullptr)
	{
		graphicsRequirements->adapterLuid = GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU()->GetAdapterLuid();
	}
	else
	{
		graphicsRequirements->adapterLuid = {};
	}
	// TODO: what do we actually want to return? The 'native' feature level or the captured level? Maybe some logical combo
	graphicsRequirements->minFeatureLevel = D3D_FEATURE_LEVEL_12_0;

	{
		FWriteScopeLock InstanceLock(InstanceMutex);
		CurrentInstance->ValidGraphicsBindingTypes.Add(XR_TYPE_GRAPHICS_BINDING_D3D12_KHR, systemId);
	}

	return XR_SUCCESS;
}
#endif

#if defined(XR_USE_GRAPHICS_API_OPENGL)
// XR_KHR_opengl_enable
XrResult FOpenXREmulationLayer::XrLayerGetOpenGLGraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsOpenGLKHR* graphicsRequirements)
{
	return UnsupportedFuncPlaceholder(EOpenXRAPIPacketId::GetOpenGLGraphicsRequirementsKHR);
}
#endif

#if defined(XR_USE_GRAPHICS_API_VULKAN)
// XR_KHR_vulkan_enable
XrResult FOpenXREmulationLayer::XrLayerGetVulkanGraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsVulkanKHR* graphicsRequirements)
{
	return UnsupportedFuncPlaceholder(EOpenXRAPIPacketId::GetVulkanGraphicsRequirementsKHR);
}
#endif


} // namespace UE::XRScribe
