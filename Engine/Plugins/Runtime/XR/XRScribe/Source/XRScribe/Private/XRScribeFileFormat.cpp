// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRScribeFileFormat.h"

// Archive operators for OpenXR API types

// OpenXR Structures
FArchive& operator<<(FArchive& Ar, XrApiLayerProperties& LayerProperty)
{
	Ar.Serialize(static_cast<void*>(&LayerProperty), sizeof(LayerProperty));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrExtensionProperties& ExtensionProperty)
{
	Ar.Serialize(static_cast<void*>(&ExtensionProperty), sizeof(ExtensionProperty));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrApplicationInfo& ApplicationInfo)
{
	Ar.Serialize(static_cast<void*>(&ApplicationInfo), sizeof(ApplicationInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrInstanceProperties& InstanceProperties)
{
	Ar.Serialize(static_cast<void*>(&InstanceProperties), sizeof(InstanceProperties));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrSystemGetInfo& SystemGetInfo)
{
	Ar.Serialize(static_cast<void*>(&SystemGetInfo), sizeof(SystemGetInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrSystemProperties& SystemProperties)
{
	Ar.Serialize(static_cast<void*>(&SystemProperties), sizeof(SystemProperties));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrSessionCreateInfo& SessionCreateInfo)
{
	Ar.Serialize(static_cast<void*>(&SessionCreateInfo), sizeof(SessionCreateInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrReferenceSpaceCreateInfo& ReferenceSpaceCreateInfo)
{
	Ar.Serialize(static_cast<void*>(&ReferenceSpaceCreateInfo), sizeof(ReferenceSpaceCreateInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrActionSpaceCreateInfo& ActionSpaceCreateInfo)
{
	Ar.Serialize(static_cast<void*>(&ActionSpaceCreateInfo), sizeof(ActionSpaceCreateInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrExtent2Df& Extent)
{
	Ar << Extent.width;
	Ar << Extent.height;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrSpaceLocation& Location)
{
	Ar.Serialize(static_cast<void*>(&Location), sizeof(Location));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrViewConfigurationProperties& ConfigurationProperties)
{
	Ar.Serialize(static_cast<void*>(&ConfigurationProperties), sizeof(ConfigurationProperties));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrViewConfigurationView& ConfigurationView)
{
	Ar.Serialize(static_cast<void*>(&ConfigurationView), sizeof(ConfigurationView));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrSwapchainCreateInfo& SwapchainCreateInfo)
{
	Ar.Serialize(static_cast<void*>(&SwapchainCreateInfo), sizeof(SwapchainCreateInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrSwapchainImageBaseHeader& Image)
{
	Ar.Serialize(static_cast<void*>(&Image), sizeof(Image));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrSwapchainImageAcquireInfo& AcquireInfo)
{
	Ar.Serialize(static_cast<void*>(&AcquireInfo), sizeof(AcquireInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrSwapchainImageWaitInfo& WaitInfo)
{
	Ar.Serialize(static_cast<void*>(&WaitInfo), sizeof(WaitInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrSwapchainImageReleaseInfo& ReleaseInfo)
{
	Ar.Serialize(static_cast<void*>(&ReleaseInfo), sizeof(ReleaseInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrSessionBeginInfo& SessionBeginInfo)
{
	Ar.Serialize(static_cast<void*>(&SessionBeginInfo), sizeof(SessionBeginInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrFrameWaitInfo& FrameWaitInfo)
{
	Ar.Serialize(static_cast<void*>(&FrameWaitInfo), sizeof(FrameWaitInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrFrameState& FrameState)
{
	Ar.Serialize(static_cast<void*>(&FrameState), sizeof(FrameState));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrFrameBeginInfo& FrameBeginInfo)
{
	Ar.Serialize(static_cast<void*>(&FrameBeginInfo), sizeof(FrameBeginInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrCompositionLayerBaseHeader& LayerBase)
{
	Ar.Serialize(static_cast<void*>(&LayerBase), sizeof(LayerBase));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrCompositionLayerQuad& LayerQuad)
{
	Ar.Serialize(static_cast<void*>(&LayerQuad), sizeof(LayerQuad));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrCompositionLayerProjection& LayerProjection)
{
	Ar.Serialize(static_cast<void*>(&LayerProjection), sizeof(LayerProjection));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrCompositionLayerProjectionView& LayerProjectionView)
{
	Ar.Serialize(static_cast<void*>(&LayerProjectionView), sizeof(LayerProjectionView));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrViewLocateInfo& ViewLocateInfo)
{
	Ar.Serialize(static_cast<void*>(&ViewLocateInfo), sizeof(ViewLocateInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrViewState& ViewState)
{
	Ar.Serialize(static_cast<void*>(&ViewState), sizeof(ViewState));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrView& View)
{
	Ar.Serialize(static_cast<void*>(&View), sizeof(View));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrActionSetCreateInfo& ActionSetCreateInfo)
{
	Ar.Serialize(static_cast<void*>(&ActionSetCreateInfo), sizeof(ActionSetCreateInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrActionSuggestedBinding& ActionSuggestedBinding)
{
	Ar.Serialize(static_cast<void*>(&ActionSuggestedBinding), sizeof(ActionSuggestedBinding));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrInteractionProfileState& InteractionProfile)
{
	Ar.Serialize(static_cast<void*>(&InteractionProfile), sizeof(InteractionProfile));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrActionStateGetInfo& GetInfo)
{
	Ar.Serialize(static_cast<void*>(&GetInfo), sizeof(GetInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrActionStateBoolean& BooleanState)
{
	Ar.Serialize(static_cast<void*>(&BooleanState), sizeof(BooleanState));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrActionStateFloat& FloatState)
{
	Ar.Serialize(static_cast<void*>(&FloatState), sizeof(FloatState));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrActionStateVector2f& Vector2fState)
{
	Ar.Serialize(static_cast<void*>(&Vector2fState), sizeof(Vector2fState));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrActionStatePose& PoseState)
{
	Ar.Serialize(static_cast<void*>(&PoseState), sizeof(PoseState));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrActiveActionSet& ActiveActionSet)
{
	Ar.Serialize(static_cast<void*>(&ActiveActionSet), sizeof(ActiveActionSet));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrBoundSourcesForActionEnumerateInfo& EnumerateInfo)
{
	Ar.Serialize(static_cast<void*>(&EnumerateInfo), sizeof(EnumerateInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrInputSourceLocalizedNameGetInfo& NameGetInfo)
{
	Ar.Serialize(static_cast<void*>(&NameGetInfo), sizeof(NameGetInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrHapticActionInfo& HapticActionInfo)
{
	Ar.Serialize(static_cast<void*>(&HapticActionInfo), sizeof(HapticActionInfo));
	return Ar;
}

// TODO: Header might not make sense...
FArchive& operator<<(FArchive& Ar, XrHapticBaseHeader& HapticFeedback)
{
	Ar.Serialize(static_cast<void*>(&HapticFeedback), sizeof(HapticFeedback));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrLoaderInitInfoBaseHeaderKHR& LoaderInitInfo)
{
	Ar.Serialize(static_cast<void*>(&LoaderInitInfo), sizeof(LoaderInitInfo));
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrVector2f& VectorStruct)
{
	Ar.Serialize(static_cast<void*>(&VectorStruct), sizeof(VectorStruct));
	return Ar;
}

#if defined(XR_USE_GRAPHICS_API_D3D11)
FArchive& operator<<(FArchive& Ar, XrGraphicsRequirementsD3D11KHR& GraphicsRequirementsD3D11)
{
	Ar.Serialize(static_cast<void*>(&GraphicsRequirementsD3D11), sizeof(GraphicsRequirementsD3D11));
	return Ar;
}
#endif

#if defined(XR_USE_GRAPHICS_API_D3D12)
FArchive& operator<<(FArchive& Ar, XrGraphicsRequirementsD3D12KHR& GraphicsRequirementsD3D12)
{
	Ar.Serialize(static_cast<void*>(&GraphicsRequirementsD3D12), sizeof(GraphicsRequirementsD3D12));
	return Ar;
}
#endif

// OpenXR Handles

FArchive& operator<<(FArchive& Ar, XrInstance& InstanceHandle)
{
	Ar << reinterpret_cast<uint64&>(InstanceHandle);
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrSession& SessionHandle)
{
	Ar << reinterpret_cast<uint64&>(SessionHandle);
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrSpace& SpaceHandle)
{
	Ar << reinterpret_cast<uint64&>(SpaceHandle);
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrAction& ActionHandle)
{
	Ar << reinterpret_cast<uint64&>(ActionHandle);
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrActionSet& ActionSetHandle)
{
	Ar << reinterpret_cast<uint64&>(ActionSetHandle);
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrSwapchain& SwapchainHandle)
{
	Ar << reinterpret_cast<uint64&>(SwapchainHandle);
	return Ar;
}

// OpenXR enums
FArchive& operator<<(FArchive& Ar, XrViewConfigurationType& ViewConfigurationType)
{
	Ar << reinterpret_cast<uint32&>(ViewConfigurationType);
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrEnvironmentBlendMode& BlendMode)
{
	Ar << reinterpret_cast<uint32&>(BlendMode);
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrReferenceSpaceType& ReferenceSpaceType)
{
	Ar << reinterpret_cast<uint32&>(ReferenceSpaceType);
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrActionType& ActionType)
{
	Ar << reinterpret_cast<uint32&>(ActionType);
	return Ar;
}

FArchive& operator<<(FArchive& Ar, XrVisibilityMaskTypeKHR& VisibilityMaskType)
{
	Ar << reinterpret_cast<uint32&>(VisibilityMaskType);
	return Ar;
}

namespace UE::XRScribe
{

FOpenXRAPIPacketBase::FOpenXRAPIPacketBase(XrResult InResult, EOpenXRAPIPacketId InApiId)
	: Result(InResult)
	, ApiId(InApiId)
	, Padding0(MagicPacketByte)
{
	TimeInCycles = FPlatformTime::Cycles64();
	ThreadId = IsInGameThread() ? EOpenXRAPIThreadId::GameThread 
			: (IsInActualRenderingThread() ? EOpenXRAPIThreadId::RenderThread 
			: EOpenXRAPIThreadId::RHIThread);
}

FArchive& operator<<(FArchive& Ar, FOpenXRAPIPacketBase& Packet)
{
	Ar.Serialize(static_cast<void*>(&Packet), sizeof(Packet));
	return Ar;
}

FOpenXREnumerateApiLayerPropertiesPacket::FOpenXREnumerateApiLayerPropertiesPacket(XrResult InResult) : 
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::EnumerateApiLayerProperties) 
{
}

FArchive& operator<<(FArchive& Ar, FOpenXREnumerateApiLayerPropertiesPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.LayerProperties;
	return Ar;
}

FOpenXREnumerateInstanceExtensionPropertiesPacket::FOpenXREnumerateInstanceExtensionPropertiesPacket(XrResult InResult) : 
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::EnumerateInstanceExtensionProperties) 
{
	LayerName[0] = '\0';
}

FArchive& operator<<(FArchive& Ar, FOpenXREnumerateInstanceExtensionPropertiesPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar.Serialize(static_cast<void*>(&Packet.LayerName[0]), sizeof(Packet.LayerName));
	Ar << Packet.ExtensionProperties;
	return Ar;
}

FOpenXRCreateInstancePacket::FOpenXRCreateInstancePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::CreateInstance), 
	CreateFlags(0), ApplicationInfo({}), GeneratedInstance(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRCreateInstancePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.CreateFlags;
	Ar << Packet.ApplicationInfo;
	Ar << Packet.EnabledLayerNames;
	Ar << Packet.EnabledExtensionNames;
	Ar << Packet.GeneratedInstance;
	return Ar;
}

FOpenXRDestroyInstancePacket::FOpenXRDestroyInstancePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::DestroyInstance),
	Instance(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRDestroyInstancePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	return Ar;
}

FOpenXRGetInstancePropertiesPacket::FOpenXRGetInstancePropertiesPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetInstanceProperties),
	Instance(XR_NULL_HANDLE), InstanceProperties({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRGetInstancePropertiesPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	Ar << Packet.InstanceProperties;
	return Ar;
}

FOpenXRGetSystemPacket::FOpenXRGetSystemPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetSystem),
	Instance(XR_NULL_HANDLE), SystemGetInfo({}), SystemId(0)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRGetSystemPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	Ar << Packet.SystemGetInfo;
	Ar << Packet.SystemId;
	return Ar;
}

FOpenXRGetSystemPropertiesPacket::FOpenXRGetSystemPropertiesPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetSystemProperties),
	Instance(XR_NULL_HANDLE), SystemId(0), SystemProperties({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRGetSystemPropertiesPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	Ar << Packet.SystemId;
	Ar << Packet.SystemProperties;
	return Ar;
}

FOpenXREnumerateEnvironmentBlendModesPacket::FOpenXREnumerateEnvironmentBlendModesPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::EnumerateEnvironmentBlendModes),
	Instance(XR_NULL_HANDLE), SystemId(0), ViewConfigurationType(XR_VIEW_CONFIGURATION_TYPE_MAX_ENUM)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXREnumerateEnvironmentBlendModesPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	Ar << Packet.SystemId;
	Ar << Packet.ViewConfigurationType;
	Ar << Packet.EnvironmentBlendModes;
	return Ar;
}

FOpenXRCreateSessionPacket::FOpenXRCreateSessionPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::CreateSession),
	Instance(XR_NULL_HANDLE), SessionCreateInfo({}), Session(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRCreateSessionPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	Ar << Packet.SessionCreateInfo;
	Ar << Packet.Session;
	return Ar;
}

FOpenXRDestroySessionPacket::FOpenXRDestroySessionPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::DestroySession),
	Session(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRDestroySessionPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	return Ar;
}

FOpenXREnumerateReferenceSpacesPacket::FOpenXREnumerateReferenceSpacesPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::EnumerateReferenceSpaces),
	Session(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXREnumerateReferenceSpacesPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.Spaces;
	return Ar;
}

FOpenXRCreateReferenceSpacePacket::FOpenXRCreateReferenceSpacePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::CreateReferenceSpace),
	Session(XR_NULL_HANDLE), ReferenceSpaceCreateInfo({}), Space(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRCreateReferenceSpacePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.ReferenceSpaceCreateInfo;
	Ar << Packet.Space;
	return Ar;
}

FOpenXRGetReferenceSpaceBoundsRectPacket::FOpenXRGetReferenceSpaceBoundsRectPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetReferenceSpaceBoundsRect),
	Session(XR_NULL_HANDLE), ReferenceSpaceType(XR_REFERENCE_SPACE_TYPE_MAX_ENUM), Bounds({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRGetReferenceSpaceBoundsRectPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.ReferenceSpaceType;
	Ar << Packet.Bounds;
	return Ar;
}

FOpenXRCreateActionSpacePacket::FOpenXRCreateActionSpacePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::CreateActionSpace),
	Session(XR_NULL_HANDLE), ActionSpaceCreateInfo({}), Space(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRCreateActionSpacePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.ActionSpaceCreateInfo;
	Ar << Packet.Space;
	return Ar;
}

FOpenXRLocateSpacePacket::FOpenXRLocateSpacePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::LocateSpace),
	Space(XR_NULL_HANDLE), BaseSpace(XR_NULL_HANDLE), Time(0), Location({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRLocateSpacePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Space;
	Ar << Packet.BaseSpace;
	Ar << Packet.Time;
	Ar << Packet.Location;
	return Ar;
}

FOpenXRDestroySpacePacket::FOpenXRDestroySpacePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::DestroySpace),
	Space(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRDestroySpacePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Space;
	return Ar;
}

FOpenXREnumerateViewConfigurationsPacket::FOpenXREnumerateViewConfigurationsPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::EnumerateViewConfigurations),
	Instance(XR_NULL_HANDLE), SystemId(0)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXREnumerateViewConfigurationsPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	Ar << Packet.SystemId;
	Ar << Packet.ViewConfigurationTypes;
	return Ar;
}

FOpenXRGetViewConfigurationPropertiesPacket::FOpenXRGetViewConfigurationPropertiesPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetViewConfigurationProperties),
	Instance(XR_NULL_HANDLE), SystemId(0), ViewConfigurationType(XR_VIEW_CONFIGURATION_TYPE_MAX_ENUM), ConfigurationProperties({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRGetViewConfigurationPropertiesPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	Ar << Packet.SystemId;
	Ar << Packet.ViewConfigurationType;
	Ar << Packet.ConfigurationProperties;
	return Ar;
}

FOpenXREnumerateViewConfigurationViewsPacket::FOpenXREnumerateViewConfigurationViewsPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::EnumerateViewConfigurationViews),
	Instance(XR_NULL_HANDLE), SystemId(0), ViewConfigurationType(XR_VIEW_CONFIGURATION_TYPE_MAX_ENUM)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXREnumerateViewConfigurationViewsPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	Ar << Packet.SystemId;
	Ar << Packet.ViewConfigurationType;
	Ar << Packet.Views;
	return Ar;
}

FOpenXREnumerateSwapchainFormatsPacket::FOpenXREnumerateSwapchainFormatsPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::EnumerateSwapchainFormats),
	Session(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXREnumerateSwapchainFormatsPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.Formats;
	return Ar;
}

FOpenXRCreateSwapchainPacket::FOpenXRCreateSwapchainPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::CreateSwapchain),
	Session(XR_NULL_HANDLE), SwapchainCreateInfo({}), Swapchain(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRCreateSwapchainPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.SwapchainCreateInfo;
	Ar << Packet.Swapchain;
	return Ar;
}

FOpenXRDestroySwapchainPacket::FOpenXRDestroySwapchainPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::DestroySwapchain),
	Swapchain(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRDestroySwapchainPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Swapchain;
	return Ar;
}

FOpenXREnumerateSwapchainImagesPacket::FOpenXREnumerateSwapchainImagesPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::EnumerateSwapchainImages),
	Swapchain(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXREnumerateSwapchainImagesPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Swapchain;
	Ar << Packet.Images;
	return Ar;
}

FOpenXRAcquireSwapchainImagePacket::FOpenXRAcquireSwapchainImagePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::AcquireSwapchainImage),
	Swapchain(XR_NULL_HANDLE), AcquireInfo({}), Index(0)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRAcquireSwapchainImagePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Swapchain;
	Ar << Packet.AcquireInfo;
	Ar << Packet.Index;
	return Ar;
}

FOpenXRWaitSwapchainImagePacket::FOpenXRWaitSwapchainImagePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::WaitSwapchainImage),
	Swapchain(XR_NULL_HANDLE), WaitInfo({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRWaitSwapchainImagePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Swapchain;
	Ar << Packet.WaitInfo;
	return Ar;
}

FOpenXRReleaseSwapchainImagePacket::FOpenXRReleaseSwapchainImagePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::ReleaseSwapchainImage),
	Swapchain(XR_NULL_HANDLE), ReleaseInfo({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRReleaseSwapchainImagePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Swapchain;
	Ar << Packet.ReleaseInfo;
	return Ar;
}

FOpenXRBeginSessionPacket::FOpenXRBeginSessionPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::BeginSession),
	Session(XR_NULL_HANDLE), SessionBeginInfo({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRBeginSessionPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.SessionBeginInfo;
	return Ar;
}

FOpenXREndSessionPacket::FOpenXREndSessionPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::EndSession),
	Session(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXREndSessionPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	return Ar;
}

FOpenXRRequestExitSessionPacket::FOpenXRRequestExitSessionPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::RequestExitSession),
	Session(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRRequestExitSessionPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	return Ar;
}

FOpenXRWaitFramePacket::FOpenXRWaitFramePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::WaitFrame),
	Session(XR_NULL_HANDLE), FrameWaitInfo({}), FrameState({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRWaitFramePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.FrameWaitInfo;
	Ar << Packet.FrameState;
	return Ar;
}

FOpenXRBeginFramePacket::FOpenXRBeginFramePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::BeginFrame),
	Session(XR_NULL_HANDLE), FrameBeginInfo({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRBeginFramePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.FrameBeginInfo;
	return Ar;
}

FOpenXREndFramePacket::FOpenXREndFramePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::EndFrame),
	Session(XR_NULL_HANDLE), DisplayTime(0), EnvironmentBlendMode(XR_ENVIRONMENT_BLEND_MODE_MAX_ENUM)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXREndFramePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.DisplayTime;
	Ar << Packet.EnvironmentBlendMode;
	Ar << Packet.Layers;
	Ar << Packet.QuadLayers;
	Ar << Packet.ProjectionLayers;
	Ar << Packet.ProjectionViews;
	return Ar;
}

FOpenXRLocateViewsPacket::FOpenXRLocateViewsPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::LocateViews),
	Session(XR_NULL_HANDLE), ViewLocateInfo({}), ViewState({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRLocateViewsPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.ViewLocateInfo;
	Ar << Packet.ViewState;
	Ar << Packet.Views;
	return Ar;
}

FOpenXRStringToPathPacket::FOpenXRStringToPathPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::StringToPath),
	Instance(XR_NULL_HANDLE), GeneratedPath(0)
{
	PathStringToWrite[0] = '\0';
}

FArchive& operator<<(FArchive& Ar, FOpenXRStringToPathPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	Ar << Packet.PathStringToWrite;
	Ar << Packet.GeneratedPath;
	return Ar;
}

FOpenXRPathToStringPacket::FOpenXRPathToStringPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::PathToString),
	Instance(XR_NULL_HANDLE), ExistingPath(0)
{
	PathStringToRead[0] = '\0';
}

FArchive& operator<<(FArchive& Ar, FOpenXRPathToStringPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	Ar << Packet.ExistingPath;
	Ar << Packet.PathStringToRead;
	return Ar;
}

FOpenXRCreateActionSetPacket::FOpenXRCreateActionSetPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::CreateActionSet),
	Instance(XR_NULL_HANDLE), ActionSetCreateInfo({}), ActionSet(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRCreateActionSetPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	Ar << Packet.ActionSetCreateInfo;
	Ar << Packet.ActionSet;
	return Ar;
}

FOpenXRDestroyActionSetPacket::FOpenXRDestroyActionSetPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::DestroyActionSet),
	ActionSet(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRDestroyActionSetPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.ActionSet;
	return Ar;
}

FOpenXRCreateActionPacket::FOpenXRCreateActionPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::CreateAction),
	ActionSet(XR_NULL_HANDLE), ActionType(XR_ACTION_TYPE_MAX_ENUM), Action(XR_NULL_HANDLE)
{
	ActionName[0] = '\0';
	LocalizedActionName[0] = '\0';
}

FArchive& operator<<(FArchive& Ar, FOpenXRCreateActionPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.ActionSet;
	Ar << Packet.ActionName;
	Ar << Packet.ActionType;
	Ar << Packet.SubactionPaths;
	Ar << Packet.LocalizedActionName;
	Ar << Packet.Action;
	return Ar;
}

FOpenXRDestroyActionPacket::FOpenXRDestroyActionPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::DestroyAction),
	Action(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRDestroyActionPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Action;
	return Ar;
}

FOpenXRSuggestInteractionProfileBindingsPacket::FOpenXRSuggestInteractionProfileBindingsPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::SuggestInteractionProfileBindings),
	Instance(XR_NULL_HANDLE), InteractionProfile(0)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRSuggestInteractionProfileBindingsPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	Ar << Packet.InteractionProfile;
	Ar << Packet.SuggestedBindings;
	return Ar;
}

FOpenXRAttachSessionActionSetsPacket::FOpenXRAttachSessionActionSetsPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::AttachSessionActionSets),
	Session(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRAttachSessionActionSetsPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.ActionSets;
	return Ar;
}

FOpenXRGetCurrentInteractionProfilePacket::FOpenXRGetCurrentInteractionProfilePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetCurrentInteractionProfile),
	Session(XR_NULL_HANDLE), TopLevelUserPath(0), InteractionProfile({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRGetCurrentInteractionProfilePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.TopLevelUserPath;
	Ar << Packet.InteractionProfile;
	return Ar;
}

FOpenXRGetActionStateBooleanPacket::FOpenXRGetActionStateBooleanPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetActionStateBoolean),
	Session(XR_NULL_HANDLE), GetInfoBoolean({}), BooleanState({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRGetActionStateBooleanPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.GetInfoBoolean;
	Ar << Packet.BooleanState;
	return Ar;
}

FOpenXRGetActionStateFloatPacket::FOpenXRGetActionStateFloatPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetActionStateFloat),
	Session(XR_NULL_HANDLE), GetInfoFloat({}), FloatState({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRGetActionStateFloatPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.GetInfoFloat;
	Ar << Packet.FloatState;
	return Ar;
}

FOpenXRGetActionStateVector2fPacket::FOpenXRGetActionStateVector2fPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetActionStateVector2F),
	Session(XR_NULL_HANDLE), GetInfoVector2f({}), Vector2fState({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRGetActionStateVector2fPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.GetInfoVector2f;
	Ar << Packet.Vector2fState;
	return Ar;
}

FOpenXRGetActionStatePosePacket::FOpenXRGetActionStatePosePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetActionStatePose),
	Session(XR_NULL_HANDLE), GetInfoPose({}), PoseState({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRGetActionStatePosePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.GetInfoPose;
	Ar << Packet.PoseState;
	return Ar;
}

FOpenXRSyncActionsPacket::FOpenXRSyncActionsPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::SyncActions),
	Session(XR_NULL_HANDLE)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRSyncActionsPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.ActiveActionSets;
	return Ar;
}

FOpenXREnumerateBoundSourcesForActionPacket::FOpenXREnumerateBoundSourcesForActionPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::EnumerateBoundSourcesForAction),
	Session(XR_NULL_HANDLE), EnumerateInfo({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXREnumerateBoundSourcesForActionPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.EnumerateInfo;
	Ar << Packet.Sources;
	return Ar;
}

FOpenXRGetInputSourceLocalizedNamePacket::FOpenXRGetInputSourceLocalizedNamePacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetInputSourceLocalizedName),
	Session(XR_NULL_HANDLE), NameGetInfo({})
{
	LocalizedName[0] = '\0';
}

FArchive& operator<<(FArchive& Ar, FOpenXRGetInputSourceLocalizedNamePacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.NameGetInfo;
	Ar << Packet.LocalizedName;
	return Ar;
}

FOpenXRApplyHapticFeedbackPacket::FOpenXRApplyHapticFeedbackPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::ApplyHapticFeedback),
	Session(XR_NULL_HANDLE), HapticActionInfo({}), HapticFeedback({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRApplyHapticFeedbackPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.HapticActionInfo;
	Ar << Packet.HapticFeedback;
	return Ar;
}

FOpenXRStopHapticFeedbackPacket::FOpenXRStopHapticFeedbackPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::StopHapticFeedback),
	Session(XR_NULL_HANDLE), HapticActionInfo({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRStopHapticFeedbackPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.HapticActionInfo;
	return Ar;
}

// XR_KHR_loader_init
FOpenXRInitializeLoaderKHRPacket::FOpenXRInitializeLoaderKHRPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::InitializeLoaderKHR),
	LoaderInitInfo({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRInitializeLoaderKHRPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.LoaderInitInfo;
	return Ar;
}

// XR_KHR_visibility_mask
FOpenXRGetVisibilityMaskKHRPacket::FOpenXRGetVisibilityMaskKHRPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetVisibilityMaskKHR),
	Session(XR_NULL_HANDLE), ViewConfigurationType(XR_VIEW_CONFIGURATION_TYPE_MAX_ENUM), 
	ViewIndex(0), VisibilityMaskType(XR_VISIBILITY_MASK_TYPE_MAX_ENUM_KHR)
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRGetVisibilityMaskKHRPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Session;
	Ar << Packet.ViewConfigurationType;
	Ar << Packet.ViewIndex;
	Ar << Packet.VisibilityMaskType;
	Ar << Packet.Vertices;
	Ar << Packet.Indices;
	return Ar;
}

#if defined(XR_USE_GRAPHICS_API_D3D11)
// XR_KHR_D3D11_enable
FOpenXRGetD3D11GraphicsRequirementsKHRPacket::FOpenXRGetD3D11GraphicsRequirementsKHRPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetD3D11GraphicsRequirementsKHR),
	Instance(XR_NULL_HANDLE), SystemId(0), GraphicsRequirementsD3D11({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRGetD3D11GraphicsRequirementsKHRPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	Ar << Packet.SystemId;
	Ar << Packet.GraphicsRequirementsD3D11;
	return Ar;
}
#endif

#if defined(XR_USE_GRAPHICS_API_D3D12)
// XR_KHR_D3D12_enable
FOpenXRGetD3D12GraphicsRequirementsKHRPacket::FOpenXRGetD3D12GraphicsRequirementsKHRPacket(XrResult InResult) :
	FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetD3D12GraphicsRequirementsKHR),
	Instance(XR_NULL_HANDLE), SystemId(0), GraphicsRequirementsD3D12({})
{
}

FArchive& operator<<(FArchive& Ar, FOpenXRGetD3D12GraphicsRequirementsKHRPacket& Packet)
{
	Ar << static_cast<FOpenXRAPIPacketBase&>(Packet);
	Ar << Packet.Instance;
	Ar << Packet.SystemId;
	Ar << Packet.GraphicsRequirementsD3D12;
	return Ar;
}
#endif

} // namespace UE::XRScribe
