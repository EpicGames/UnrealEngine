// Copyright Epic Games, Inc. All Rights Reserved.

#include "XRScribeModule.h"

#include "Modules/ModuleManager.h"
#include "XRScribeAPISurface.h"

IMPLEMENT_MODULE(FXRScribeModule, XRScribe)

void FXRScribeModule::StartupModule()
{
	RegisterOpenXRExtensionModularFeature();
	IModularFeatures::Get().RegisterModularFeature(GetFeatureName(), this);

	UE::XRScribe::BuildFunctionMaps();
}

void FXRScribeModule::ShutdownModule()
{
	UnregisterOpenXRExtensionModularFeature();
	IModularFeatures::Get().UnregisterModularFeature(GetFeatureName(), this);

	UE::XRScribe::ClearFunctionMaps();
	ChainedGetProcAddr = nullptr;
}

FString FXRScribeModule::GetDisplayName()
{
	return GetFeatureName().ToString();
}

bool FXRScribeModule::InsertOpenXRAPILayer(PFN_xrGetInstanceProcAddr& InOutGetProcAddr)
{
	// Ok to pass nullptr for GetProcAddr because that would only disable capture. Replay still valid!
	const bool bResult = UE::XRScribe::IOpenXRAPILayerManager::Get().SetChainedGetProcAddr(InOutGetProcAddr);

	if (bResult)
	{
		ChainedGetProcAddr = InOutGetProcAddr;
		InOutGetProcAddr = &UE::XRScribe::xrGetInstanceProcAddr;
	}

	return bResult;
}


FXRScribeModule* FXRScribeModule::Get()
{
	TArray<FXRScribeModule*> Impls = IModularFeatures::Get().GetModularFeatureImplementations<FXRScribeModule>(GetFeatureName());

	check(Impls.Num() <= 1);

	if (Impls.Num() > 0)
	{
		check(Impls[0]);
		return Impls[0];
	}
	return nullptr;
}

