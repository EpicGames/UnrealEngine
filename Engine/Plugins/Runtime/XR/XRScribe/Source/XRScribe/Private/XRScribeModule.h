// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "IOpenXRExtensionPlugin.h"
#include "Modules/ModuleInterface.h"
#include "UObject/NameTypes.h"

class FXRScribeModule : public IModuleInterface, public IOpenXRExtensionPlugin
{
public:
	/************************************************************************/
	/* IModuleInterface                                                     */
	/************************************************************************/
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/************************************************************************/
	/* IOpenXRExtensionPlugin                                               */
	/************************************************************************/
	virtual FString GetDisplayName() override;
	virtual bool InsertOpenXRAPILayer(PFN_xrGetInstanceProcAddr& InOutGetProcAddr) override;

	/************************************************************************/
	/* FXRScribe                                                             */
	/************************************************************************/
	static FXRScribeModule* Get();

private:
	static FName GetFeatureName()
	{
		static FName FeatureName = FName(TEXT("XRScribe"));
		return FeatureName;
	}

private:
	PFN_xrGetInstanceProcAddr ChainedGetProcAddr = nullptr;
};
