// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenXRPlatformRHI.h"
#include "Containers/Map.h"
#include "UObject/NameTypes.h"
#include "Serialization/ArrayReader.h"
#include "XRScribeFileFormat.h"


DEFINE_LOG_CATEGORY_STATIC(LogXRScribeEmulate, Log, All);

namespace UE::XRScribe
{

class FOpenXRCaptureDecoder
{
public:
	explicit FOpenXRCaptureDecoder();
	~FOpenXRCaptureDecoder();

	bool DecodeDataFromMemory();

	// state accessors
	[[nodiscard]] const TArray<XrExtensionProperties>& GetInstanceExtensionProperties() { return InstanceExtensionProperties; }
	[[nodiscard]] const TArray<XrApiLayerProperties>& GetApiLayerProperties() {	return ApiLayerProperties; }
	[[nodiscard]] XrInstanceCreateFlags GetInstanceCreateFlags() { return ValidInstanceCreateFlags; }
	[[nodiscard]] const TArray<TStaticArray<ANSICHAR, XR_MAX_API_LAYER_NAME_SIZE>>& GetRequestedApiLayerNames() { return RequestedLayerNames; }
	[[nodiscard]] const TArray<TStaticArray<ANSICHAR, XR_MAX_EXTENSION_NAME_SIZE>>& GetRequestedExtensionNames() { return RequestedExtensionNames; }
	[[nodiscard]] const XrInstanceProperties& GetInstanceProperties() { return InstanceProperties; }
	[[nodiscard]] const XrSystemGetInfo& GetSystemInfo() { return SystemGetInfo; }
	[[nodiscard]] const XrSystemProperties& GetSystemProperties() { return SystemProperties; }
	[[nodiscard]] const TArray<XrEnvironmentBlendMode>& GetEnvironmentBlendModes() { return EnvironmentBlendModes; }
	[[nodiscard]] const TArray<XrViewConfigurationType>& GetViewConfigurationTypes() { return ViewConfigurationTypes; }
	[[nodiscard]] const TMap<XrViewConfigurationType, XrViewConfigurationProperties>& GetViewConfigurationProperties() { return ViewConfigurationProperties; }
	[[nodiscard]] const TMap<XrViewConfigurationType, TArray<XrViewConfigurationView>>& GetViewConfigurationViews() { return ViewConfigurationViews; }
	[[nodiscard]] const TMap<XrViewConfigurationType, TArray<FOpenXRLocateViewsPacket>>& GetViewLocations() { return ViewLocations; }
	[[nodiscard]] const TArray<FOpenXRCreateReferenceSpacePacket>& GetCreatedReferenceSpaces() { return CreatedReferenceSpaces; }
	[[nodiscard]] const TArray<FOpenXRCreateActionSpacePacket>& GetCreatedActionSpaces() { return CreatedActionSpaces; }
	[[nodiscard]] const TMap<XrSpace, TArray<FOpenXRLocateSpacePacket>>& GetSpaceLocations() { return SpaceLocations; }
	[[nodiscard]] const TArray<XrReferenceSpaceType>& GetReferenceSpaceTypes() { return ReferenceSpaceTypes; }
	[[nodiscard]] const TMap<XrReferenceSpaceType, XrExtent2Df>& GetReferenceSpaceBounds() { return ReferenceSpaceBounds; }
	[[nodiscard]] const TArray<int64>& GetSwapchainFormats() { return SwapchainFormats; }
	[[nodiscard]] const TArray<FOpenXRCreateActionPacket>& GetCreatedActions() { return CreatedActions; }
	[[nodiscard]] const TArray<FOpenXRWaitFramePacket>& GetWaitFrames() { return WaitFrames; }
	[[nodiscard]] const TArray<FOpenXRSyncActionsPacket>& GetSyncActions() { return SyncActions; }
	[[nodiscard]] const TMap<XrAction, TArray<FOpenXRGetActionStateBooleanPacket>>& GetBooleanActionStates() { return BooleanActionStates; }
	[[nodiscard]] const TMap<XrAction, TArray<FOpenXRGetActionStateFloatPacket>>& GetFloatActionStates() { return FloatActionStates; }
	[[nodiscard]] const TMap<XrAction, TArray<FOpenXRGetActionStateVector2fPacket>>& GetVectorActionStates() { return VectorActionStates; }
	[[nodiscard]] const TMap<XrAction, TArray<FOpenXRGetActionStatePosePacket>>& GetPoseActionStates() { return PoseActionStates; }
	[[nodiscard]] const TMap<XrPath, FName>& GetPathToStringMap() { return PathToStringMap; }

	TArray<uint8>& GetEncodedData()
	{
		return EncodedData;
	}

protected:

	// packet decoders
	bool DecodeEnumerateApiLayerProperties(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeEnumerateInstanceExtensionProperties(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeCreateInstance(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeDestroyInstance(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeGetInstanceProperties(const FOpenXRAPIPacketBase& BasePacket);
	//bool DecodePollEvent(const FOpenXRAPIPacketBase& BasePacket);
	//bool DecodeResultToString(const FOpenXRAPIPacketBase& BasePacket);
	//bool DecodeStructureTypeToString(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeGetSystem(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeGetSystemProperties(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeEnumerateEnvironmentBlendModes(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeCreateSession(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeDestroySession(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeEnumerateReferenceSpaces(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeCreateReferenceSpace(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeGetReferenceSpaceBoundsRect(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeCreateActionSpace(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeLocateSpace(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeDestroySpace(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeEnumerateViewConfigurations(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeGetViewConfigurationProperties(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeEnumerateViewConfigurationViews(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeEnumerateSwapchainFormats(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeCreateSwapchain(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeDestroySwapchain(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeEnumerateSwapchainImages(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeAcquireSwapchainImage(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeWaitSwapchainImage(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeReleaseSwapchainImage(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeBeginSession(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeEndSession(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeRequestExitSession(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeWaitFrame(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeBeginFrame(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeEndFrame(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeLocateViews(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeStringToPath(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodePathToString(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeCreateActionSet(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeDestroyActionSet(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeCreateAction(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeDestroyAction(const FOpenXRAPIPacketBase& BasePacket);

	bool DecodeSuggestInteractionProfileBindings(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeAttachSessionActionSets(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeGetCurrentInteractionProfile(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeGetActionStateBoolean(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeGetActionStateFloat(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeGetActionStateVector2f(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeGetActionStatePose(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeSyncActions(const FOpenXRAPIPacketBase& BasePacket);
	//bool DecodeEnumerateBoundSourcesForAction(const FOpenXRAPIPacketBase& BasePacket);
	//bool DecodeGetInputSourceLocalizedName(const FOpenXRAPIPacketBase& BasePacket);

	bool DecodeApplyHapticFeedback(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeStopHapticFeedback(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeInitializeLoaderKHR(const FOpenXRAPIPacketBase& BasePacket);
	bool DecodeGetVisibilityMaskKHR(const FOpenXRAPIPacketBase& BasePacket);
#if defined(XR_USE_GRAPHICS_API_D3D11)
	bool DecodeGetD3D11GraphicsRequirementsKHR(const FOpenXRAPIPacketBase& BasePacket);
#endif
#if defined(XR_USE_GRAPHICS_API_D3D12)
	bool DecodeGetD3D12GraphicsRequirementsKHR(const FOpenXRAPIPacketBase& BasePacket);
#endif

	FArrayReader EncodedData;

	typedef bool(FOpenXRCaptureDecoder::* ApiDecodeFn)(const FOpenXRAPIPacketBase& BasePacket);
	TStaticArray<ApiDecodeFn, (uint32)EOpenXRAPIPacketId::NumValidAPIPacketIds> DecodeFnTable;

	// derived state from capture

	TArray<XrExtensionProperties> InstanceExtensionProperties;
	TArray<XrApiLayerProperties> ApiLayerProperties;
	// TODO: Per-layer extension properties

	XrInstanceCreateFlags ValidInstanceCreateFlags = 0;
	TArray<TStaticArray<ANSICHAR, XR_MAX_API_LAYER_NAME_SIZE>> RequestedLayerNames;
	TArray<TStaticArray<ANSICHAR, XR_MAX_EXTENSION_NAME_SIZE>> RequestedExtensionNames;

	XrInstanceProperties InstanceProperties = {};

	XrSystemGetInfo SystemGetInfo = {};
	XrSystemProperties SystemProperties = {};
	TArray<XrEnvironmentBlendMode> EnvironmentBlendModes;

	XrSessionCreateInfo SessionCreateInfo = {};

	TArray<XrReferenceSpaceType> ReferenceSpaceTypes;
	TMap<XrReferenceSpaceType, XrExtent2Df> ReferenceSpaceBounds;
	TMap<XrSpace, XrReferenceSpaceType> ReferenceSpaceMap;
	TArray<FOpenXRCreateReferenceSpacePacket> CreatedReferenceSpaces;

	TArray<FOpenXRCreateActionSpacePacket> CreatedActionSpaces;
	TMap<XrSpace, XrAction> ActionSpaceMap;

	TMap<XrSpace, TArray<FOpenXRLocateSpacePacket>> SpaceLocations;

	TArray<XrViewConfigurationType> ViewConfigurationTypes;
	TMap<XrViewConfigurationType, XrViewConfigurationProperties> ViewConfigurationProperties;
	TMap<XrViewConfigurationType, TArray<XrViewConfigurationView>> ViewConfigurationViews;

	TArray<int64> SwapchainFormats;

	TMap<XrViewConfigurationType, TArray<FOpenXRLocateViewsPacket>> ViewLocations;

	TMap<XrPath, FName> PathToStringMap;
	TMap<FName, TArray<XrActionSuggestedBinding>> StringToSuggestedBindingsMap;

	TArray<FOpenXRCreateActionPacket> CreatedActions;

	TArray<FOpenXRWaitFramePacket> WaitFrames;

	TArray<FOpenXRSyncActionsPacket> SyncActions;

	TMap<XrAction, TArray<FOpenXRGetActionStateBooleanPacket>> BooleanActionStates;
	TMap<XrAction, TArray<FOpenXRGetActionStateFloatPacket>> FloatActionStates;
	TMap<XrAction, TArray<FOpenXRGetActionStateVector2fPacket>> VectorActionStates;
	TMap<XrAction, TArray<FOpenXRGetActionStatePosePacket>> PoseActionStates;

	// TODO: Would I ever want to bin properties into per-instance collections?
	// When we are repaying, we're just going to create our own set of 'valid' parameters
	// that our replay runtime supports, do we really care about how the original application run
	// managed their instances?
};

}
