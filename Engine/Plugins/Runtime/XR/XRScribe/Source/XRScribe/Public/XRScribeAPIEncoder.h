// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ScopeRWLock.h"
#include "OpenXRPlatformRHI.h"
#include "Serialization/BufferArchive.h"

// TODO Might be useful for writing a tool to manually modify an existing capture

namespace UE::XRScribe
{

class FOpenXRCaptureEncoder : public TBufferArchive<32>
{
public:
	FOpenXRCaptureEncoder();
	~FOpenXRCaptureEncoder();

	// Core

	// Global
	void EncodeEnumerateApiLayerProperties(const XrResult Result, const uint32_t propertyCapacityInput, const uint32_t* propertyCountOutput, const XrApiLayerProperties* properties);
	void EncodeEnumerateInstanceExtensionProperties(const XrResult Result, const char* layerName, const uint32_t propertyCapacityInput, const uint32_t* propertyCountOutput, const XrExtensionProperties* properties);
	void EncodeCreateInstance(const XrResult Result, const XrInstanceCreateInfo* createInfo, const XrInstance* instance);
	
	// Instance
	void EncodeDestroyInstance(const XrResult Result, const XrInstance instance);
	void EncodeGetInstanceProperties(const XrResult Result, const XrInstance instance, const XrInstanceProperties* instanceProperties);

	void EncodePollEvent(const XrResult Result, const XrInstance instance, const XrEventDataBuffer* eventData);

	void EncodeResultToString(const XrResult Result, const XrInstance instance, const XrResult value, const char buffer[XR_MAX_RESULT_STRING_SIZE]);
	void EncodeStructureTypeToString(const XrResult Result, const XrInstance instance, const XrStructureType value, const char buffer[XR_MAX_STRUCTURE_NAME_SIZE]);

	void EncodeGetSystem(const XrResult Result, const XrInstance instance, const XrSystemGetInfo* getInfo, const XrSystemId* systemId);
	void EncodeGetSystemProperties(const XrResult Result, const XrInstance instance, const XrSystemId systemId, const XrSystemProperties* properties);
	void EncodeEnumerateEnvironmentBlendModes(const XrResult Result, const XrInstance instance, const XrSystemId systemId, const XrViewConfigurationType viewConfigurationType, const uint32_t environmentBlendModeCapacityInput, const uint32_t* environmentBlendModeCountOutput, const XrEnvironmentBlendMode* environmentBlendModes);

	void EncodeCreateSession(const XrResult Result, const XrInstance instance, const XrSessionCreateInfo* createInfo, const XrSession* session);
	void EncodeDestroySession(const XrResult Result, const XrSession session);

	void EncodeEnumerateReferenceSpaces(const XrResult Result, const XrSession session, const uint32_t spaceCapacityInput, const uint32_t* spaceCountOutput, const XrReferenceSpaceType* spaces);
	void EncodeCreateReferenceSpace(const XrResult Result, const XrSession session, const XrReferenceSpaceCreateInfo* createInfo, const XrSpace* space);
	void EncodeGetReferenceSpaceBoundsRect(const XrResult Result, const XrSession session, const XrReferenceSpaceType referenceSpaceType, const XrExtent2Df* bounds);
	void EncodeCreateActionSpace(const XrResult Result, const XrSession session, const XrActionSpaceCreateInfo* createInfo, const XrSpace* space);
	void EncodeLocateSpace(const XrResult Result, const XrSpace space, const XrSpace baseSpace, const XrTime time, const XrSpaceLocation* location);
	void EncodeDestroySpace(const XrResult Result, const XrSpace space);

	void EncodeEnumerateViewConfigurations(const XrResult Result, const XrInstance instance, const XrSystemId systemId, const uint32_t viewConfigurationTypeCapacityInput, const uint32_t* viewConfigurationTypeCountOutput, const XrViewConfigurationType* viewConfigurationTypes);
	void EncodeGetViewConfigurationProperties(const XrResult Result, const XrInstance instance, const XrSystemId systemId, const XrViewConfigurationType viewConfigurationType, const XrViewConfigurationProperties* configurationProperties);
	void EncodeEnumerateViewConfigurationViews(const XrResult Result, const XrInstance instance, const XrSystemId systemId, const XrViewConfigurationType viewConfigurationType, const uint32_t viewCapacityInput, const uint32_t* viewCountOutput, const XrViewConfigurationView* views);

	void EncodeEnumerateSwapchainFormats(const XrResult Result, const XrSession session, const uint32_t formatCapacityInput, const uint32_t* formatCountOutput, int64_t* formats);
	void EncodeCreateSwapchain(const XrResult Result, const XrSession session, const XrSwapchainCreateInfo* createInfo, const XrSwapchain* swapchain);
	void EncodeDestroySwapchain(const XrResult Result, const XrSwapchain swapchain);
	void EncodeEnumerateSwapchainImages(const XrResult Result, const XrSwapchain swapchain, const uint32_t imageCapacityInput, const uint32_t* imageCountOutput, const XrSwapchainImageBaseHeader* images);
	void EncodeAcquireSwapchainImage(const XrResult Result, const XrSwapchain swapchain, const XrSwapchainImageAcquireInfo* acquireInfo, const uint32_t* index);
	void EncodeWaitSwapchainImage(const XrResult Result, const XrSwapchain swapchain, const XrSwapchainImageWaitInfo* waitInfo);
	void EncodeReleaseSwapchainImage(const XrResult Result, const XrSwapchain swapchain, const XrSwapchainImageReleaseInfo* releaseInfo);

	void EncodeBeginSession(const XrResult Result, const XrSession session, const XrSessionBeginInfo* beginInfo);
	void EncodeEndSession(const XrResult Result, const XrSession session);
	void EncodeRequestExitSession(const XrResult Result, const XrSession session);

	void EncodeWaitFrame(const XrResult Result, const XrSession session, const XrFrameWaitInfo* frameWaitInfo, const XrFrameState* frameState);
	void EncodeBeginFrame(const XrResult Result, const XrSession session, const XrFrameBeginInfo* frameBeginInfo);
	void EncodeEndFrame(const XrResult Result, const XrSession session, const XrFrameEndInfo* frameEndInfo);
	void EncodeLocateViews(const XrResult Result, const XrSession session, const XrViewLocateInfo* viewLocateInfo, const XrViewState* viewState, const uint32_t viewCapacityInput, const uint32_t* viewCountOutput, const XrView* views);

	void EncodeStringToPath(const XrResult Result, const XrInstance instance, const char* pathString, const XrPath* path);
	void EncodePathToString(const XrResult Result, const XrInstance instance, const XrPath path, const uint32_t bufferCapacityInput, const uint32_t* bufferCountOutput, const char* buffer);

	void EncodeCreateActionSet(const XrResult Result, const XrInstance instance, const XrActionSetCreateInfo* createInfo, const XrActionSet* actionSet);
	void EncodeDestroyActionSet(const XrResult Result, const XrActionSet actionSet);
	void EncodeCreateAction(const XrResult Result, const XrActionSet actionSet, const XrActionCreateInfo* createInfo, const XrAction* action);
	void EncodeDestroyAction(const XrResult Result, const XrAction action);

	void EncodeSuggestInteractionProfileBindings(const XrResult Result, const XrInstance instance, const XrInteractionProfileSuggestedBinding* suggestedBindings);
	void EncodeAttachSessionActionSets(const XrResult Result, const XrSession session, const XrSessionActionSetsAttachInfo* attachInfo);
	void EncodeGetCurrentInteractionProfile(const XrResult Result, const XrSession session, const XrPath topLevelUserPath, const XrInteractionProfileState* interactionProfile);

	void EncodeGetActionStateBoolean(const XrResult Result, const XrSession session, const XrActionStateGetInfo* getInfo, const XrActionStateBoolean* state);
	void EncodeGetActionStateFloat(const XrResult Result, const XrSession session, const XrActionStateGetInfo* getInfo, const XrActionStateFloat* state);
	void EncodeGetActionStateVector2f(const XrResult Result, const XrSession session, const XrActionStateGetInfo* getInfo, const XrActionStateVector2f* state);
	void EncodeGetActionStatePose(const XrResult Result, const XrSession session, const XrActionStateGetInfo* getInfo, const XrActionStatePose* state);
	void EncodeSyncActions(const XrResult Result, const XrSession session, const	XrActionsSyncInfo* syncInfo);
	
	void EncodeEnumerateBoundSourcesForAction(const XrResult Result, const XrSession session, const XrBoundSourcesForActionEnumerateInfo* enumerateInfo, const uint32_t sourceCapacityInput, const uint32_t* sourceCountOutput, const XrPath* sources);
	void EncodeGetInputSourceLocalizedName(const XrResult Result, const XrSession session, const XrInputSourceLocalizedNameGetInfo* getInfo, const uint32_t bufferCapacityInput, const uint32_t* bufferCountOutput, const char* buffer);

	void EncodeApplyHapticFeedback(const XrResult Result, const XrSession session, const XrHapticActionInfo* hapticActionInfo, const XrHapticBaseHeader* hapticFeedback);
	void EncodeStopHapticFeedback(const XrResult Result, const XrSession session, const XrHapticActionInfo* hapticActionInfo);
	
	void EncodeInitializeLoaderKHR(const XrResult Result, const XrLoaderInitInfoBaseHeaderKHR* loaderInitInfo);

	void EncodeGetVisibilityMaskKHR(const XrResult Result, const XrSession session, const XrViewConfigurationType viewConfigurationType, const uint32_t viewIndex, const XrVisibilityMaskTypeKHR visibilityMaskType, const XrVisibilityMaskKHR* visibilityMask);

#if defined(XR_USE_GRAPHICS_API_D3D11)
	void EncodeGetD3D11GraphicsRequirementsKHR(const XrResult Result, const XrInstance instance, const XrSystemId systemId, const XrGraphicsRequirementsD3D11KHR* graphicsRequirements);
#endif
#if defined(XR_USE_GRAPHICS_API_D3D12)
	void EncodeGetD3D12GraphicsRequirementsKHR(const XrResult Result, const XrInstance instance, const XrSystemId systemId, const XrGraphicsRequirementsD3D12KHR* graphicsRequirements);
#endif

protected:

	template <typename PacketType>
	void WritePacketData(PacketType& Data);

	FRWLock CaptureWriteMutex;
};

} // namespace UE::XRScribe
