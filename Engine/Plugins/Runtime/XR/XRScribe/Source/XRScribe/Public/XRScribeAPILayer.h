// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicRHI.h"
#include "OpenXRPlatformRHI.h"

namespace UE::XRScribe
{

class IOpenXRAPILayer
{
public:
	////
	// Infrastructure functions
	////

	virtual ~IOpenXRAPILayer() {}
	virtual void SetChainedGetProcAddr(PFN_xrGetInstanceProcAddr InChainedGetProcAddr) {};
	virtual bool SupportsInstanceExtension(const ANSICHAR* ExtensionName) = 0;


	////
	// Layer functions
	////

	// Global
	virtual XrResult XrLayerEnumerateApiLayerProperties(uint32_t propertyCapacityInput, uint32_t* propertyCountOutput, XrApiLayerProperties* properties) = 0;
	virtual XrResult XrLayerEnumerateInstanceExtensionProperties(const char* layerName, uint32_t propertyCapacityInput, uint32_t* propertyCountOutput, XrExtensionProperties* properties) = 0;
	virtual XrResult XrLayerCreateInstance(const XrInstanceCreateInfo* createInfo, XrInstance* instance) = 0;

	// Instance
	virtual XrResult XrLayerDestroyInstance(XrInstance instance) = 0;
	virtual XrResult XrLayerGetInstanceProperties(XrInstance instance, XrInstanceProperties* instanceProperties) = 0;
	virtual XrResult XrLayerPollEvent(XrInstance instance, XrEventDataBuffer* eventData) = 0;
	virtual XrResult XrLayerResultToString(XrInstance instance, XrResult value, char buffer[XR_MAX_RESULT_STRING_SIZE]) = 0;
	virtual XrResult XrLayerStructureTypeToString(XrInstance instance, XrStructureType value, char buffer[XR_MAX_STRUCTURE_NAME_SIZE]) = 0;
	virtual XrResult XrLayerGetSystem(XrInstance instance, const XrSystemGetInfo* getInfo, XrSystemId* systemId) = 0;
	virtual XrResult XrLayerGetSystemProperties(XrInstance instance, XrSystemId systemId, XrSystemProperties* properties) = 0;
	virtual XrResult XrLayerEnumerateEnvironmentBlendModes(XrInstance instance, XrSystemId systemId, XrViewConfigurationType viewConfigurationType, uint32_t environmentBlendModeCapacityInput, uint32_t* environmentBlendModeCountOutput, XrEnvironmentBlendMode* environmentBlendModes) = 0;
	virtual XrResult XrLayerCreateSession(XrInstance instance, const XrSessionCreateInfo* createInfo, XrSession* session) = 0;
	virtual XrResult XrLayerDestroySession(XrSession session) = 0;
	virtual XrResult XrLayerEnumerateReferenceSpaces(XrSession session, uint32_t spaceCapacityInput, uint32_t* spaceCountOutput, XrReferenceSpaceType* spaces) = 0;
	virtual XrResult XrLayerCreateReferenceSpace(XrSession session, const XrReferenceSpaceCreateInfo* createInfo, XrSpace* space) = 0;
	virtual XrResult XrLayerGetReferenceSpaceBoundsRect(XrSession session, XrReferenceSpaceType referenceSpaceType, XrExtent2Df* bounds) = 0;
	virtual XrResult XrLayerCreateActionSpace(XrSession session, const XrActionSpaceCreateInfo* createInfo, XrSpace* space) = 0;
	virtual XrResult XrLayerLocateSpace(XrSpace space, XrSpace baseSpace, XrTime time, XrSpaceLocation* location) = 0;
	virtual XrResult XrLayerDestroySpace(XrSpace space) = 0;
	virtual XrResult XrLayerEnumerateViewConfigurations(XrInstance instance, XrSystemId systemId, uint32_t viewConfigurationTypeCapacityInput, uint32_t* viewConfigurationTypeCountOutput, XrViewConfigurationType* viewConfigurationTypes) = 0;
	virtual XrResult XrLayerGetViewConfigurationProperties(XrInstance instance, XrSystemId systemId, XrViewConfigurationType viewConfigurationType, XrViewConfigurationProperties* configurationProperties) = 0;
	virtual XrResult XrLayerEnumerateViewConfigurationViews(XrInstance instance, XrSystemId systemId, XrViewConfigurationType viewConfigurationType, uint32_t viewCapacityInput, uint32_t* viewCountOutput, XrViewConfigurationView* views) = 0;
	virtual XrResult XrLayerEnumerateSwapchainFormats(XrSession session, uint32_t formatCapacityInput, uint32_t* formatCountOutput, int64_t* formats) = 0;
	virtual XrResult XrLayerCreateSwapchain(XrSession	session, const XrSwapchainCreateInfo* createInfo, XrSwapchain* swapchain) = 0;
	virtual XrResult XrLayerDestroySwapchain(XrSwapchain swapchain) = 0;
	virtual XrResult XrLayerEnumerateSwapchainImages(XrSwapchain swapchain, uint32_t imageCapacityInput, uint32_t* imageCountOutput, XrSwapchainImageBaseHeader* images) = 0;
	virtual XrResult XrLayerAcquireSwapchainImage(XrSwapchain swapchain, const XrSwapchainImageAcquireInfo* acquireInfo, uint32_t* index) = 0;
	virtual XrResult XrLayerWaitSwapchainImage(XrSwapchain swapchain, const XrSwapchainImageWaitInfo* waitInfo) = 0;
	virtual XrResult XrLayerReleaseSwapchainImage(XrSwapchain swapchain, const XrSwapchainImageReleaseInfo* releaseInfo) = 0;
	virtual XrResult XrLayerBeginSession(XrSession session, const XrSessionBeginInfo* beginInfo) = 0;
	virtual XrResult XrLayerEndSession(XrSession session) = 0;
	virtual XrResult XrLayerRequestExitSession(XrSession session) = 0;
	virtual XrResult XrLayerWaitFrame(XrSession session, const XrFrameWaitInfo* frameWaitInfo, XrFrameState* frameState) = 0;
	virtual XrResult XrLayerBeginFrame(XrSession session, const XrFrameBeginInfo* frameBeginInfo) = 0;
	virtual XrResult XrLayerEndFrame(XrSession session, const XrFrameEndInfo* frameEndInfo) = 0;
	virtual XrResult XrLayerLocateViews(XrSession session, const XrViewLocateInfo* viewLocateInfo, XrViewState* viewState, uint32_t viewCapacityInput, uint32_t* viewCountOutput, XrView* views) = 0;
	virtual XrResult XrLayerStringToPath(XrInstance instance, const char* pathString, XrPath* path) = 0;
	virtual XrResult XrLayerPathToString(XrInstance instance, XrPath path, uint32_t bufferCapacityInput, uint32_t* bufferCountOutput, char* buffer) = 0;
	virtual XrResult XrLayerCreateActionSet(XrInstance instance, const XrActionSetCreateInfo* createInfo, XrActionSet* actionSet) = 0;
	virtual XrResult XrLayerDestroyActionSet(XrActionSet actionSet) = 0;
	virtual XrResult XrLayerCreateAction(XrActionSet actionSet, const XrActionCreateInfo* createInfo, XrAction* action) = 0;
	virtual XrResult XrLayerDestroyAction(XrAction action) = 0;
	virtual XrResult XrLayerSuggestInteractionProfileBindings(XrInstance instance, const XrInteractionProfileSuggestedBinding* suggestedBindings) = 0;
	virtual XrResult XrLayerAttachSessionActionSets(XrSession session, const XrSessionActionSetsAttachInfo* attachInfo) = 0;
	virtual XrResult XrLayerGetCurrentInteractionProfile(XrSession session, XrPath topLevelUserPath, XrInteractionProfileState* interactionProfile) = 0;
	virtual XrResult XrLayerGetActionStateBoolean(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStateBoolean* state) = 0;
	virtual XrResult XrLayerGetActionStateFloat(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStateFloat* state) = 0;
	virtual XrResult XrLayerGetActionStateVector2f(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStateVector2f* state) = 0;
	virtual XrResult XrLayerGetActionStatePose(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStatePose* state) = 0;
	virtual XrResult XrLayerSyncActions(XrSession session, const	XrActionsSyncInfo* syncInfo) = 0;
	virtual XrResult XrLayerEnumerateBoundSourcesForAction(XrSession session, const XrBoundSourcesForActionEnumerateInfo* enumerateInfo, uint32_t sourceCapacityInput, uint32_t* sourceCountOutput, XrPath* sources) = 0;
	virtual XrResult XrLayerGetInputSourceLocalizedName(XrSession session, const XrInputSourceLocalizedNameGetInfo* getInfo, uint32_t bufferCapacityInput, uint32_t* bufferCountOutput, char* buffer) = 0;
	virtual XrResult XrLayerApplyHapticFeedback(XrSession session, const XrHapticActionInfo* hapticActionInfo, const XrHapticBaseHeader* hapticFeedback) = 0;
	virtual XrResult XrLayerStopHapticFeedback(XrSession session, const XrHapticActionInfo* hapticActionInfo) = 0;

	// Global extensions

	// XR_KHR_loader_init
	virtual XrResult XrLayerInitializeLoaderKHR(const XrLoaderInitInfoBaseHeaderKHR* loaderInitInfo) = 0;

	// Instance extensions

	// XR_KHR_visibility_mask
	virtual XrResult XrLayerGetVisibilityMaskKHR(XrSession session, XrViewConfigurationType viewConfigurationType, uint32_t viewIndex, XrVisibilityMaskTypeKHR visibilityMaskType, XrVisibilityMaskKHR* visibilityMask) = 0;

#if defined(XR_USE_GRAPHICS_API_D3D11)
	// XR_KHR_D3D11_enable
	virtual XrResult XrLayerGetD3D11GraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsD3D11KHR* graphicsRequirements) = 0;
#endif

#if defined(XR_USE_GRAPHICS_API_D3D12)
	// XR_KHR_D3D12_enable
	virtual XrResult XrLayerGetD3D12GraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsD3D12KHR* graphicsRequirements) = 0;
#endif

#if defined(XR_USE_GRAPHICS_API_OPENGL)
	// XR_KHR_opengl_enable
	virtual XrResult XrLayerGetOpenGLGraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsOpenGLKHR* graphicsRequirements) = 0;
#endif

#if defined(XR_USE_GRAPHICS_API_VULKAN)
	// XR_KHR_vulkan_enable
	virtual XrResult XrLayerGetVulkanGraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsVulkanKHR* graphicsRequirements) = 0;
#endif

// TODO extra instance extension funcs
//xrCreateHandTrackerEXT
//xrDestroyHandTrackerEXT
//xrLocateHandJointsEXT
//xrConvertWin32PerformanceCounterToTimeKHR

};

} // namespace UE::XRScribe