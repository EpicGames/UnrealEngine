// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <openxr/openxr.h>

namespace UE::XRScribe
{

class IOpenXRAPILayer;

void BuildFunctionMaps();
void ClearFunctionMaps();

XRAPI_ATTR XrResult XRAPI_CALL xrGetInstanceProcAddr(
	XrInstance  instance,
	const char* name,
	PFN_xrVoidFunction* function);

class IOpenXRAPILayerManager
{
public:
	virtual IOpenXRAPILayer* GetActiveLayer() const = 0;

	virtual void SetFallbackRunMode(int32_t Fallback) = 0;

	virtual bool SetChainedGetProcAddr(PFN_xrGetInstanceProcAddr InChainedGetProcAddr) = 0;
	
	static IOpenXRAPILayerManager& Get();
};

} // namespace UE::XRScribe
