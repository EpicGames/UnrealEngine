// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Because we initialize XRScribe before developer settings are read in from the config, this setting is just used
// to set XRScribe.RunMode in DefaultEngine.ini, where it'll be read in directly from the config file in DetermineRunMode().
// This differs from the normal method of pulling settings from the backing CVars or using the UXRScribeDeveloperSettings 
// class default object.

#include "Engine/DeveloperSettings.h"
#include "XRScribeDeveloperSettings.generated.h"

/**
 * Enumerates available options for XRScribe run mode.
 */
UENUM()
namespace EXRScribeRunMode
{
	enum Type : int
	{
		Capture = 0 UMETA(ToolTip = "Capture OpenXR API calls and store to Saved/Capture.xrs."),
		Emulate = 1 UMETA(ToolTip = "Emulate OpenXR runtime and playback Saved/Capture.xrs."),
	};
}

/** Developer settings for XRScribe */
UCLASS(config = Engine, defaultconfig, meta = (DisplayName = "XRScribe"))
class XRSCRIBE_API UXRScribeDeveloperSettings : public UDeveloperSettings
{
	GENERATED_BODY()
public:

	UXRScribeDeveloperSettings(const FObjectInitializer& Initializer);

	/** Controls whether XRScribe runs in capture or emulation mode. Currently needed at engine startup, but will be runtime switchable. */
	UPROPERTY(config, EditAnywhere, Category = "XRScribe", meta = (
		ConsoleVariable = "XRScribe.RunMode", DisplayName = "Run Mode",
		ToolTip = "0 - Capture, 1 - Emulate",
		ConfigRestartRequired = true))
	TEnumAsByte<EXRScribeRunMode::Type> RunMode = EXRScribeRunMode::Type::Emulate; // This should match the default FallbackRunMode set in XRScribeAPISurface.cpp

	// TODO:
	// File path for capture file
	// Customizing capture dump point (session end, instance teardown, app end)
	// other run modes (e.g. replay)

public:
	virtual FName GetCategoryName() const;
};