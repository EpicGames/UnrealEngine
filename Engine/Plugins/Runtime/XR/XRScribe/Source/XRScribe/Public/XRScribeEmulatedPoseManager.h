// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "XRScribeFileFormat.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StaticArray.h"
#include "Logging/LogMacros.h"
#include "Math/Transform.h"
#include "OpenXRCore.h"

namespace UE::XRScribe
{

// The ActionPoseManager has two primary jobs:
// * ingest data from capture
// * generate estimated poses/actions based on captured state + timings
//
// When using the ActionPoseManager, we'll interact with it similarly to xrLocateSpace: give it time and spaces,
// and return locations. On the Action side, give it an action and sync time, and we return an action state.
//
// Internally, we'll plan to have all managed poses in tracker-space, and convert to the different spaces
// as needed. This does mean we'll need to know how to convert between different spaces, even if the
// original capture might not tell us how to

class FOpenXRActionPoseManager
{

public:

	// Number of poses the manager will keep available relative to 'current' pose, in order for multi-frame pose queries
	static const int32 PoseHistorySize = 5;

	/**
	 * Register information about paths to path strings from the capture
	 *
	 * @param PathStringMap Captured map of path handles to FNames
	 *
	 */
	void RegisterCapturedPathStrings(const TMap<XrPath, FName>& PathStringMap);

	/**
	 * Register information about wait frames from a capture file, in order to build a history of
	 * display times
	 *
	 * @param InWaitFrameHistory The WaitFrame packets from the capture
	 *
	 */
	void RegisterCapturedWaitFrames(const TArray<FOpenXRWaitFramePacket>& InWaitFrameHistory);

	/**
	 * Register information about reference spaces from a capture file, in order to build a history of
	 * space locations relative to those reference spaces.
	 *
	 * @param CreateReferenceSpacePackets The create infos for the captured reference spaces
	 *
	 */
	void RegisterCapturedReferenceSpaces(const TArray<FOpenXRCreateReferenceSpacePacket>& CreateReferenceSpacePackets);

	/**
	 * Register information about actions from capture file, in order to fetch action name for mapping between capture and emulation
	 *
	 * @param CreateActionPackets The create infos for the captured actions
	 *
	 */
	void RegisterCapturedActions(const TArray<FOpenXRCreateActionPacket>& CreateActionPackets);


	/**
	 * Register information about action spaces from capture file, which we need to build a history of space locations
	 * for XR devices
	 *
	 * @param CreateActionSpacePackets The create infos for the captured action spaces
	 *
	 */
	void RegisterCapturedActionSpaces(const TArray<FOpenXRCreateActionSpacePacket>& CreateActionSpacePackets);

	/**
	 * Register history of LocateSpace calls
	 *
	 * @param SpaceHistories List of LocateSpace histories, with a unified Space per history (but possibly varying BaseSpace) 
	 *
	 */
	void RegisterCapturedSpaceHistories(const TMap<XrSpace, TArray<FOpenXRLocateSpacePacket>>& SpaceHistories);


	/**
	 * Register history of GetActionState + SyncActions calls
	 *
	 * @param SyncActionsPackets List of SyncActions calls
	 * @param BooleanActionStates Per-action history of boolean action states
	 * @param FloatActionStates Per-action history of float action states
	 * @param VectorActionStates Per-action history of Vector2f action states
	 * @param PoseActionStates Per-action history of Pose action states (indicating whether pose is valid or not, not actual pose data)
	 *
	 */
	void RegisterCapturedActionStates(const TArray<FOpenXRSyncActionsPacket>& SyncActionsPackets, 
		const TMap<XrAction, TArray<FOpenXRGetActionStateBooleanPacket>>& BooleanActionStates,
		const TMap<XrAction, TArray<FOpenXRGetActionStateFloatPacket>>& FloatActionStates,
		const TMap<XrAction, TArray<FOpenXRGetActionStateVector2fPacket>>& VectorActionStates,
		const TMap<XrAction, TArray<FOpenXRGetActionStatePosePacket>>& PoseActionStates);

	/**
	 * Process all captured pose and action state histories at once, in order to create a unified timeline
	 *
	 */
	void ProcessCapturedHistories();

	/**
	 * Register emulated path. Manager needs this to match paths between capture and emulation.
	 *
	 * @param PathString String used to create path
	 * @param Path Emulation-created path handle
	 *
	 */
	void RegisterEmulatedPath(FName PathString, XrPath Path);

	/**
	 * Register emulated action. Manager needs this to match emulated actions against captured action state histories
	 *
	 * @param ActionName String used for action name
	 * @param Action Emulation-created action handle
	 * @param ActionType Action-type to facilitate history type lookup
	 *
	 */
	void RegisterEmulatedAction(FName ActionName, XrAction Action, XrActionType ActionType);

	/**
	 * Register emulated reference space. Pose manager needs this to match the new emulated reference space against
	 * a captured reference space of similar attributes.
	 *
	 * @param CreateInfo Create info for emulated reference space, in order to match against captured history of similarly configured space
	 * @param SpaceHandle Emulation generated handle for new space
	 *
	 */
	void RegisterEmulatedReferenceSpace(const XrReferenceSpaceCreateInfo& CreateInfo, XrSpace SpaceHandle);

	/**
	 * Register emulated action space. Pose manager needs this to match emulated action space against
	 * a captured action space with matching action name.
	 *
	 * @param ActionName Name from action create info
	 * @param SpaceCreateInfo Create info for emulated action space, in order to match against captured history of similarly configured space
	 * @param SpaceHandle Emulation generated handle for new space
	 *
	 */
	void RegisterEmulatedActionSpace(TStaticArray<ANSICHAR, XR_MAX_ACTION_NAME_SIZE>& ActionName, const XrActionSpaceCreateInfo& SpaceCreateInfo, XrSpace SpaceHandle);

	/**
	 * Add time and frame number in order to generate interpolated poses
	 * We use the frame numbers to index into a relative history of the poses, and the time
	 * for frame matching + interpolation, because LocateSpace calls only provide a time argument
	 *
	 * @param Time Emulation generated time for new frame
	 * @param FrameNum Emulation generated frame index
	 *
	 */
	void AddEmulatedFrameTime(XrTime Time, int32 FrameNum);

	/**
	 * If we have a pose history for the action, then xrGetActionStatePose can
	 * tell the client that the action is active, and poses are ready to fetch
	 *
	 * @param ActionName Name from action create info
	 * 
	 * return ActionName has available pose history
	 */
	bool DoesActionContainPoseHistory(TStaticArray<ANSICHAR, XR_MAX_ACTION_NAME_SIZE>& ActionName);

	/**
	 * Fetch a pose generated from the captured history, by using space handles and times generated by the emulated runtime
	 *
	 * @param LocatingSpace Generated pose is associated with this space
	 * @param BaseSpace Generated pose is relative to this space
	 * @param Time Time of requested pose, relative to the known times from the emulated runtime
	 *
	 * return Location derived from captured history if available, or default location if request is unknown or invalid
	 */
	XrSpaceLocation GetEmulatedPoseForTime(XrSpace LocatingSpace, XrSpace BaseSpace, XrTime Time);

	/**
	 * Fetch an action state generated from the captured history, by using XrActionStateGetInfo and most recent sync time
	 * The function itself is used to designate between different action state histories (bool, float, vector2f). 
	 *
	 * @param GetInfo provides an action handle and subpath
	 * @param LastSyncTime The sync time is managed by the emulation layer, and used to offset into the captured history.
	 *
	 * return Action state derived from captured history if available, or default inactive state if action/subpath aren't available.
	 */
	XrActionStateBoolean GetEmulatedActionStateBoolean(const XrActionStateGetInfo* GetInfo, XrTime LastSyncTime);
	XrActionStateFloat GetEmulatedActionStateFloat(const XrActionStateGetInfo* GetInfo, XrTime LastSyncTime);
	XrActionStateVector2f GetEmulatedActionStateVector2f(const XrActionStateGetInfo* GetInfo, XrTime LastSyncTime);

	/**
	 * Reset internal emulation state based on session teardown.
	 *
	 */
	void OnSessionTeardown()
	{
		EmulatedBaseTime = INT64_MAX;
	}
	
	// TODO: add LocateViews to pose manager tracking

protected:
	
	template <typename StateType>
	struct TActionState
	{
		StateType State{};
		XrBool32 Active = 0;
	};

	template <typename StateType>
	struct TActionStateHistory
	{
		TArray<TActionState<StateType>> StateValues;
		TSet<FName> ValidSubpaths;
	};

	using FBooleanActionState = TActionState<XrBool32>;
	using FFloatActionState = TActionState<float>;
	using FVector2fActionState = TActionState<XrVector2f>;

	using FBooleanActionStateHistory = TActionStateHistory<XrBool32>;
	using FFloatActionStateHistory = TActionStateHistory<float>;
	using FVector2fActionStateHistory = TActionStateHistory<XrVector2f>;

	/**
	 * Helper function to fetch underlying reference space type from opaque XrSpace handle
	 */
	XrReferenceSpaceType GetCapturedReferenceSpaceType(XrSpace CapturedSpace);

	/**
	 * Helper function to validate captured space is an action space
	 */
	bool VerifyCapturedActionSpace(XrSpace CapturedSpace);

	/**
	 * Helper function to determine full time range of capture
	 */
	void CalculateCapturedTimeRange();

	/**
	 * sort and remove duplicates in history for single space
	 */
	bool FilterSpaceHistory(const TArray<FOpenXRLocateSpacePacket>& RawHistory, TArray<FOpenXRLocateSpacePacket>& FilteredHistory);

	/**
	 * Resample the filtered history in order to ease reading poses back based on time offsets during emulation
	 */
	TArray<FOpenXRLocateSpacePacket> SliceFilteredSpaceHistory(const TArray<FOpenXRLocateSpacePacket>& FilteredHistory);

	void ProcessCapturedReferenceSpaceHistory(const TArray<FOpenXRLocateSpacePacket>& SpaceHistory);
	void ProcessCapturedActionSpaceHistory(const TArray<FOpenXRLocateSpacePacket>& SpaceHistory);

	FName GetEmulatedActionName(XrAction Action);

	/**
	 * Helper function to generate index into captured history from emulated time (offset from emulated session start)
	 */
	int64 GenerateReplaySliceIndexFromTime(XrTime EmulatedTime);

	bool ValidateSubpath(const TSet<FName>& ValidSubpaths, XrPath ActionSubPath);

	/**
	 * Helper function to managed 'cached' action states, because we're only supposed to update on state changes
	 */
	const XrActionStateBoolean GetCachedEmulatedBooleanState(FName ActionName);
	void SetCachedEmulatedBooleanState(FName ActionName, const XrActionStateBoolean& EmulatedState);
	const XrActionStateFloat GetCachedEmulatedFloatState(FName ActionName);
	void SetCachedEmulatedFloatState(FName ActionName, const XrActionStateFloat& EmulatedState);
	const XrActionStateVector2f GetCachedEmulatedVector2fState(FName ActionName);
	void SetCachedEmulatedVector2fState(FName ActionName, const XrActionStateVector2f& EmulatedState);

	TMap<XrPath, FName> CapturedPaths;
	TMap<XrPath, FName> EmulatedPaths;

	TMap<XrAction, FName> EmulatedActionNames;
	TMap<FName, XrActionStateBoolean> CachedEmulatedBooleanStates;
	TMap<FName, XrActionStateFloat> CachedEmulatedFloatStates;
	TMap<FName, XrActionStateVector2f> CachedEmulatedVector2fStates;

	// Known spaces and relevant actions from capture
	TArray<FOpenXRCreateReferenceSpacePacket> CapturedReferenceSpaces;
	TArray<FOpenXRCreateActionSpacePacket> CapturedActionSpaces;
	TMap<XrAction, FOpenXRCreateActionPacket> CapturedActions;

	TMap<XrSpace, TArray<FOpenXRLocateSpacePacket>> CapturedRawSpaceHistories;

	//TArray<FOpenXRSyncActionsPacket> CapturedRawSyncActionsPackets;
	TMap<XrAction, TArray<FOpenXRGetActionStateBooleanPacket>> CapturedRawBooleanActionStates;
	TMap<XrAction, TArray<FOpenXRGetActionStateFloatPacket>> CapturedRawFloatActionStates;
	TMap<XrAction, TArray<FOpenXRGetActionStateVector2fPacket>> CapturedRawVectorActionStates;
	TMap<XrAction, TArray<FOpenXRGetActionStatePosePacket>> CapturedRawPoseActionStates;

	/**
	 * Processed pose histories for space types, which could be unified across different space handles
	 * with the same underlying reference space type.
	 */
	TMap<XrReferenceSpaceType, TArray<FOpenXRLocateSpacePacket>> ReferencePoseHistories;
	TMap<FName, TArray<FOpenXRLocateSpacePacket>> ActionPoseHistories;

	TMap<FName, TArray<FOpenXRGetActionStateBooleanPacket>> BooleanActionStateHistories;
	TMap<FName, TArray<FOpenXRGetActionStateFloatPacket>> FloatActionStateHistories;
	TMap<FName, TArray<FOpenXRGetActionStateVector2fPacket>> VectorActionStateHistories;
	TMap<FName, TArray<FOpenXRGetActionStatePosePacket>> PoseActionStateHistories;

	TMap<FName, FBooleanActionStateHistory>  BooleanProcessedHistories;
	TMap<FName, FFloatActionStateHistory>	 FloatProcessedHistories;
	TMap<FName, FVector2fActionStateHistory> Vector2fProcessedHistories;

	// Map of emulated spaces to their underlying reference space type (if applicable)
	TMap<XrSpace, XrReferenceSpaceType> EmulatedReferenceSpaceTypeMap;

	TMap< XrSpace, FName> EmulatedActionSpaceNameMap;

	struct FrameTimeHistoryEntry
	{
		XrTime Time = 0;
		int32 FrameIndex = -1;
	};

	// History of frame times, in order for pose manager to look into to estimate the relative frame index for times
	TStaticArray <FrameTimeHistoryEntry, PoseHistorySize> FrameTimeHistory;
	int32 LastInsertedFrameIndex = 0;

	int64 CapturedSliceCount = 0;
	TArray<FOpenXRWaitFramePacket> WaitFrameHistory;

	XrTime CapturedRangeStart = 0;
	XrTime CapturedRangeEnd = 0;

	XrTime EmulatedBaseTime = INT64_MAX;
};

} // namespace UE::XRScribe
