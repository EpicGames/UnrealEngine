// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/UniquePtr.h"
#include "Containers/SpscQueue.h"
#include "Misc/ScopeRWLock.h"
#include "XRScribeAPIDecoder.h"
#include "XRScribeAPILayer.h"
#include "XRScribeEmulatedPoseManager.h"

namespace UE::XRScribe
{

struct FOpenXRUpdateEvent;

union FOpenXREmulatedGraphicsBinding;

struct FOpenXREmulatedInstance;
struct FOpenXREmulatedSession;

class FOpenXREmulationLayer : public IOpenXRAPILayer
{
public:

	// The constructor + destructor need to be defined in cpp file in order for TUniquePtr of forward decl to work (FOpenXREmulatedInstance)
	FOpenXREmulationLayer();
	virtual ~FOpenXREmulationLayer() override;

	virtual bool SupportsInstanceExtension(const ANSICHAR* ExtensionName) override;

	bool LoadCaptureFromFile(const FString& EmulationLoadPath);
	bool LoadCaptureFromData(const TArray<uint8>& EncodedData);

	// Global
	virtual XrResult XrLayerEnumerateApiLayerProperties(uint32_t propertyCapacityInput, uint32_t* propertyCountOutput, XrApiLayerProperties* properties) override;
	virtual XrResult XrLayerEnumerateInstanceExtensionProperties(const char* layerName, uint32_t propertyCapacityInput, uint32_t* propertyCountOutput, XrExtensionProperties* properties) override;
	virtual XrResult XrLayerCreateInstance(const XrInstanceCreateInfo* createInfo, XrInstance* instance) override;

	// Instance
	virtual XrResult XrLayerDestroyInstance(XrInstance instance) override;
	virtual XrResult XrLayerGetInstanceProperties(XrInstance instance, XrInstanceProperties* instanceProperties) override;
	virtual XrResult XrLayerPollEvent(XrInstance instance, XrEventDataBuffer* eventData) override;
	virtual XrResult XrLayerResultToString(XrInstance instance, XrResult value, char buffer[XR_MAX_RESULT_STRING_SIZE]) override;
	virtual XrResult XrLayerStructureTypeToString(XrInstance instance, XrStructureType value, char buffer[XR_MAX_STRUCTURE_NAME_SIZE]) override;
	virtual XrResult XrLayerGetSystem(XrInstance instance, const XrSystemGetInfo* getInfo, XrSystemId* systemId) override;
	virtual XrResult XrLayerGetSystemProperties(XrInstance instance, XrSystemId systemId, XrSystemProperties* properties) override;
	virtual XrResult XrLayerEnumerateEnvironmentBlendModes(XrInstance instance, XrSystemId systemId, XrViewConfigurationType viewConfigurationType, uint32_t environmentBlendModeCapacityInput, uint32_t* environmentBlendModeCountOutput, XrEnvironmentBlendMode* environmentBlendModes) override;
	virtual XrResult XrLayerCreateSession(XrInstance instance, const XrSessionCreateInfo* createInfo, XrSession* session) override;
	virtual XrResult XrLayerDestroySession(XrSession session) override;
	virtual XrResult XrLayerEnumerateReferenceSpaces(XrSession session, uint32_t spaceCapacityInput, uint32_t* spaceCountOutput, XrReferenceSpaceType* spaces) override;
	virtual XrResult XrLayerCreateReferenceSpace(XrSession session, const XrReferenceSpaceCreateInfo* createInfo, XrSpace* space) override;
	virtual XrResult XrLayerGetReferenceSpaceBoundsRect(XrSession session, XrReferenceSpaceType referenceSpaceType, XrExtent2Df* bounds) override;
	virtual XrResult XrLayerCreateActionSpace(XrSession session, const XrActionSpaceCreateInfo* createInfo, XrSpace* space) override;
	virtual XrResult XrLayerLocateSpace(XrSpace space, XrSpace baseSpace, XrTime time, XrSpaceLocation* location) override;
	virtual XrResult XrLayerDestroySpace(XrSpace space) override;
	virtual XrResult XrLayerEnumerateViewConfigurations(XrInstance instance, XrSystemId systemId, uint32_t viewConfigurationTypeCapacityInput, uint32_t* viewConfigurationTypeCountOutput, XrViewConfigurationType* viewConfigurationTypes) override;
	virtual XrResult XrLayerGetViewConfigurationProperties(XrInstance instance, XrSystemId systemId, XrViewConfigurationType viewConfigurationType, XrViewConfigurationProperties* configurationProperties) override;
	virtual XrResult XrLayerEnumerateViewConfigurationViews(XrInstance instance, XrSystemId systemId, XrViewConfigurationType viewConfigurationType, uint32_t viewCapacityInput, uint32_t* viewCountOutput, XrViewConfigurationView* views) override;
	virtual XrResult XrLayerEnumerateSwapchainFormats(XrSession session, uint32_t formatCapacityInput, uint32_t* formatCountOutput, int64_t* formats) override;
	virtual XrResult XrLayerCreateSwapchain(XrSession	session, const XrSwapchainCreateInfo* createInfo, XrSwapchain* swapchain) override;
	virtual XrResult XrLayerDestroySwapchain(XrSwapchain swapchain) override;
	virtual XrResult XrLayerEnumerateSwapchainImages(XrSwapchain swapchain, uint32_t imageCapacityInput, uint32_t* imageCountOutput, XrSwapchainImageBaseHeader* images) override;
	virtual XrResult XrLayerAcquireSwapchainImage(XrSwapchain swapchain, const XrSwapchainImageAcquireInfo* acquireInfo, uint32_t* index) override;
	virtual XrResult XrLayerWaitSwapchainImage(XrSwapchain swapchain, const XrSwapchainImageWaitInfo* waitInfo) override;
	virtual XrResult XrLayerReleaseSwapchainImage(XrSwapchain swapchain, const XrSwapchainImageReleaseInfo* releaseInfo) override;
	virtual XrResult XrLayerBeginSession(XrSession session, const XrSessionBeginInfo* beginInfo) override;
	virtual XrResult XrLayerEndSession(XrSession session) override;
	virtual XrResult XrLayerRequestExitSession(XrSession session) override;
	virtual XrResult XrLayerWaitFrame(XrSession session, const XrFrameWaitInfo* frameWaitInfo, XrFrameState* frameState) override;
	virtual XrResult XrLayerBeginFrame(XrSession session, const XrFrameBeginInfo* frameBeginInfo) override;
	virtual XrResult XrLayerEndFrame(XrSession session, const XrFrameEndInfo* frameEndInfo) override;
	virtual XrResult XrLayerLocateViews(XrSession session, const XrViewLocateInfo* viewLocateInfo, XrViewState* viewState, uint32_t viewCapacityInput, uint32_t* viewCountOutput, XrView* views) override;
	virtual XrResult XrLayerStringToPath(XrInstance instance, const char* pathString, XrPath* path) override;
	virtual XrResult XrLayerPathToString(XrInstance instance, XrPath path, uint32_t bufferCapacityInput, uint32_t* bufferCountOutput, char* buffer) override;
	virtual XrResult XrLayerCreateActionSet(XrInstance instance, const XrActionSetCreateInfo* createInfo, XrActionSet* actionSet) override;
	virtual XrResult XrLayerDestroyActionSet(XrActionSet actionSet) override;
	virtual XrResult XrLayerCreateAction(XrActionSet actionSet, const XrActionCreateInfo* createInfo, XrAction* action) override;
	virtual XrResult XrLayerDestroyAction(XrAction action) override;
	virtual XrResult XrLayerSuggestInteractionProfileBindings(XrInstance instance, const XrInteractionProfileSuggestedBinding* suggestedBindings) override;
	virtual XrResult XrLayerAttachSessionActionSets(XrSession session, const XrSessionActionSetsAttachInfo* attachInfo) override;
	virtual XrResult XrLayerGetCurrentInteractionProfile(XrSession session, XrPath topLevelUserPath, XrInteractionProfileState* interactionProfile) override;
	virtual XrResult XrLayerGetActionStateBoolean(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStateBoolean* state) override;
	virtual XrResult XrLayerGetActionStateFloat(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStateFloat* state) override;
	virtual XrResult XrLayerGetActionStateVector2f(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStateVector2f* state) override;
	virtual XrResult XrLayerGetActionStatePose(XrSession session, const XrActionStateGetInfo* getInfo, XrActionStatePose* state) override;
	virtual XrResult XrLayerSyncActions(XrSession session, const	XrActionsSyncInfo* syncInfo) override;
	virtual XrResult XrLayerEnumerateBoundSourcesForAction(XrSession session, const XrBoundSourcesForActionEnumerateInfo* enumerateInfo, uint32_t sourceCapacityInput, uint32_t* sourceCountOutput, XrPath* sources) override;
	virtual XrResult XrLayerGetInputSourceLocalizedName(XrSession session, const XrInputSourceLocalizedNameGetInfo* getInfo, uint32_t bufferCapacityInput, uint32_t* bufferCountOutput, char* buffer) override;
	virtual XrResult XrLayerApplyHapticFeedback(XrSession session, const XrHapticActionInfo* hapticActionInfo, const XrHapticBaseHeader* hapticFeedback) override;
	virtual XrResult XrLayerStopHapticFeedback(XrSession session, const XrHapticActionInfo* hapticActionInfo) override;

	// Global extensions

	// XR_KHR_loader_init
	virtual XrResult XrLayerInitializeLoaderKHR(const XrLoaderInitInfoBaseHeaderKHR* loaderInitInfo) override;

	// Instance extensions

	// XR_KHR_visibility_mask
	virtual XrResult XrLayerGetVisibilityMaskKHR(XrSession session, XrViewConfigurationType viewConfigurationType, uint32_t viewIndex, XrVisibilityMaskTypeKHR visibilityMaskType, XrVisibilityMaskKHR* visibilityMask) override;

#if defined(XR_USE_GRAPHICS_API_D3D11)
	// XR_KHR_D3D11_enable
	virtual XrResult XrLayerGetD3D11GraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsD3D11KHR* graphicsRequirements) override;
#endif

#if defined(XR_USE_GRAPHICS_API_D3D12)
	// XR_KHR_D3D12_enable
	virtual XrResult XrLayerGetD3D12GraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsD3D12KHR* graphicsRequirements) override;
#endif

#if defined(XR_USE_GRAPHICS_API_OPENGL)
	// XR_KHR_opengl_enable
	virtual XrResult XrLayerGetOpenGLGraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsOpenGLKHR* graphicsRequirements) override;
#endif

#if defined(XR_USE_GRAPHICS_API_VULKAN)
	// XR_KHR_vulkan_enable
	virtual XrResult XrLayerGetVulkanGraphicsRequirementsKHR(XrInstance instance, XrSystemId systemId, XrGraphicsRequirementsVulkanKHR* graphicsRequirements) override;
#endif

protected:
	void PostLoadActions();

	bool InstanceHandleCheck(XrInstance InputInstance);
	bool SystemIdCheck(XrSystemId InputSystemId);
	bool SessionHandleCheck(XrSession InputSession);

	bool SpaceHandleCheck(XrSpace InputSpace);
	bool SwapchainHandleCheck(XrSwapchain InputSwapchain);
	bool ActionSetHandleCheck(XrActionSet InputActionSet);
	bool ActionHandleCheck(XrAction InputAction);
	
	bool IsCurrentSessionRunning();

	void UpdateSessionState(FOpenXREmulatedSession& Session, XrSessionState AfterState);
	XrResult SetupGraphicsBinding(const XrSystemId SystemId, const void* InBinding, TUniquePtr<FOpenXREmulatedSession>& Session);

	EPixelFormat ConvertPlatformFormat(int64 PlatformFormat);
	static ETextureCreateFlags ConvertXrSwapchainFlagsToTextureCreateFlags(const XrSwapchainCreateInfo* CreateInfo);

	int64 StartTimeTicks = 0;
	int64 LastSyncTimeTicks = 0;

	FOpenXRCaptureDecoder CaptureDecoder;

	TArray<XrApiLayerProperties> SupportedEmulatedLayers;
	TArray<XrExtensionProperties> SupportedEmulatedExtensions;

	FRWLock	InstanceMutex;
	TUniquePtr<FOpenXREmulatedInstance> CurrentInstance;
	XrInstanceProperties EmulatedInstanceProperties = {};

	const XrSystemId MagicSystemId = 0xdeadbeef;
	XrSystemProperties EmulatedSystemProperties = {};

	FRWLock	SessionMutex;
	TUniquePtr<FOpenXREmulatedSession> CurrentSession;

	//TMpscQueue<FOpenXRUpdateEvent> InternalUpdateQueue;
	TSpscQueue<FOpenXRUpdateEvent> PendingApplicationUpdateQueue;
	
	FOpenXRActionPoseManager ActionPoseManager;
};

} // namespace UE::XRScribe
