// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OpenXRPlatformRHI.h"
#include "Containers/Array.h"
#include "Containers/StaticArray.h"
#include "Containers/UnrealString.h"
#include "Serialization/Archive.h"

// OpenXR types operator<< implementation has to be in the global namespace
FArchive& operator<<(FArchive& Ar, XrApiLayerProperties& LayerProperty);
FArchive& operator<<(FArchive& Ar, XrExtensionProperties& ExtensionProperty);
FArchive& operator<<(FArchive& Ar, XrSystemGetInfo& SystemGetInfo);
FArchive& operator<<(FArchive& Ar, XrSystemProperties& SystemProperties);

namespace UE::XRScribe
{

enum class EOpenXRAPIThreadId : uint8
{
	GameThread = 0,
	RenderThread = 1,
	RHIThread = 2,
	// TODO: other thread types as engine APIs evolve?
};

enum class EOpenXRAPIPacketId : uint8
{
	// Core

	// Global
	//GetInstanceProcAddr = 0,
	EnumerateApiLayerProperties = 0,
	EnumerateInstanceExtensionProperties,
	CreateInstance,

	// Instance
	DestroyInstance,
	GetInstanceProperties,
	PollEvent,
	ResultToString,
	StructureTypeToString,
	GetSystem,
	GetSystemProperties,
	EnumerateEnvironmentBlendModes,
	CreateSession,
	DestroySession,
	EnumerateReferenceSpaces,
	CreateReferenceSpace,
	GetReferenceSpaceBoundsRect,
	CreateActionSpace,
	LocateSpace,
	DestroySpace,
	EnumerateViewConfigurations,
	GetViewConfigurationProperties,
	EnumerateViewConfigurationViews,
	EnumerateSwapchainFormats,
	CreateSwapchain,
	DestroySwapchain,
	EnumerateSwapchainImages,
	AcquireSwapchainImage,
	WaitSwapchainImage,
	ReleaseSwapchainImage,
	BeginSession,
	EndSession,
	RequestExitSession,
	WaitFrame,
	BeginFrame,
	EndFrame,
	LocateViews,
	StringToPath,
	PathToString,
	CreateActionSet,
	DestroyActionSet,
	CreateAction,
	DestroyAction,
	SuggestInteractionProfileBindings,
	AttachSessionActionSets,
	GetCurrentInteractionProfile,
	GetActionStateBoolean,
	GetActionStateFloat,
	GetActionStateVector2F,
	GetActionStatePose,
	SyncActions,
	EnumerateBoundSourcesForAction,
	GetInputSourceLocalizedName,
	ApplyHapticFeedback,
	StopHapticFeedback,

	// Extensions

	// Global
	InitializeLoaderKHR,

	// Instance
	GetVisibilityMaskKHR,
	PerfSettingsSetPerformanceLevelEXT,
	ThermalGetTemperatureTrendEXT,
	SetDebugUtilsObjectNameEXT,
	CreateDebugUtilsMessengerEXT,
	DestroyDebugUtilsMessengerEXT,
	SubmitDebugUtilsMessageEXT,
	SessionBeginDebugUtilsLabelRegionEXT,
	SessionEndDebugUtilsLabelRegionEXT,
	SessionInsertDebugUtilsLabelEXT,
	CreateSpatialAnchorMsft,
	CreateSpatialAnchorSpaceMsft,
	DestroySpatialAnchorMsft,
	SetInputDeviceActiveEXT,
	SetInputDeviceStateBoolEXT,
	SetInputDeviceStateFloatEXT,
	SetInputDeviceStateVector2FEXT,
	SetInputDeviceLocationEXT,
	CreateSpatialGraphNodeSpaceMsft,
	CreateHandTrackerEXT,
	DestroyHandTrackerEXT,
	LocateHandJointsEXT,
	CreateHandMeshSpaceMsft,
	UpdateHandMeshMsft,
	GetControllerModelKeyMsft,
	LoadControllerModelMsft,
	GetControllerModelPropertiesMsft,
	GetControllerModelStateMsft,
	EnumerateReprojectionModesMsft,
	UpdateSwapchainFb,
	GetSwapchainStateFb,
	EnumerateSceneComputeFeaturesMsft,
	CreateSceneObserverMsft,
	DestroySceneObserverMsft,
	CreateSceneMsft,
	DestroySceneMsft,
	ComputeNewSceneMsft,
	GetSceneComputeStateMsft,
	GetSceneComponentsMsft,
	LocateSceneComponentsMsft,
	GetSceneMeshBuffersMsft,
	DeserializeSceneMsft,
	GetSerializedSceneFragmentDataMsft,
	EnumerateDisplayRefreshRatesFb,
	GetDisplayRefreshRateFb,
	RequestDisplayRefreshRateFb,
	EnumerateColorSpacesFb,
	SetColorSpaceFb,
	SetEnvironmentDepthEstimationVarjo,

	// Platform Instance
	SetAndroidApplicationThreadKHR,
	CreateSwapchainAndroidSurfaceKHR,
	GetOpenGLGraphicsRequirementsKHR,
	GetOpenGLESGraphicsRequirementsKHR,
	GetVulkanInstanceExtensionsKHR,
	GetVulkanDeviceExtensionsKHR,
	GetVulkanGraphicsDeviceKHR,
	GetVulkanGraphicsRequirementsKHR,
	CreateVulkanInstanceKHR,
	CreateVulkanDeviceKHR,
	GetVulkanGraphicsDevice2KHR,
	GetVulkanGraphicsRequirements2KHR,
	GetD3D11GraphicsRequirementsKHR,
	GetD3D12GraphicsRequirementsKHR,
	ConvertWin32PerformanceCounterToTimeKHR,
	ConvertTimeToWin32PerformanceCounterKHR,
	CreateSpatialAnchorFromPerceptionAnchorMSFT,
	TryGetPerceptionAnchorFromSpatialAnchorMSFT,
	GetAudioOutputDeviceGuidOculus,
	GetAudioInputDeviceGuidOculus,
	ConvertTimespecTimeToTimeKHR,
	ConvertTimeToTimespecTimeKHR,

	NumValidAPIPacketIds,
};

struct FOpenXRAPIPacketBase
{
public:
	static constexpr uint8 MagicPacketByte = 0xab;

	uint64 TimeInCycles;
	XrResult Result;
	EOpenXRAPIPacketId ApiId;
	EOpenXRAPIThreadId ThreadId;

	uint16 Padding0;

	friend FArchive& operator<<(FArchive& Ar, FOpenXRAPIPacketBase& Packet);

protected:
	FOpenXRAPIPacketBase(XrResult InResult, EOpenXRAPIPacketId InApiId);


	FOpenXRAPIPacketBase() = delete;
};
static_assert(16 == sizeof(FOpenXRAPIPacketBase), "FOpenXRAPIPacketBase is unexpected size!");

//struct FOpenXRGetInstanceProcAddrPacket : public FOpenXRAPIPacketBase
//{
//	XrInstance Instance;
//	EOpenXRAPIPacketId RequestedProcAddrId;
//
//	FOpenXRGetInstanceProcAddrPacket(XrResult InResult) : FOpenXRAPIPacketBase(InResult, EOpenXRAPIPacketId::GetInstanceProcAddr) {}
//};

struct FOpenXREnumerateApiLayerPropertiesPacket : public FOpenXRAPIPacketBase
{
	TArray<XrApiLayerProperties> LayerProperties;

	FOpenXREnumerateApiLayerPropertiesPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXREnumerateApiLayerPropertiesPacket& Packet);
};

struct FOpenXREnumerateInstanceExtensionPropertiesPacket : public FOpenXRAPIPacketBase
{
	TStaticArray<ANSICHAR, XR_MAX_API_LAYER_NAME_SIZE> LayerName;
	TArray<XrExtensionProperties> ExtensionProperties;

	FOpenXREnumerateInstanceExtensionPropertiesPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXREnumerateInstanceExtensionPropertiesPacket& Packet);
};

struct FOpenXRCreateInstancePacket : public FOpenXRAPIPacketBase
{
	// relevant pieces of XrInstanceCreateInfo
	//typedef struct XrInstanceCreateInfo {
	//	XrStructureType             type;
	//	const void* XR_MAY_ALIAS    next;
	//	XrInstanceCreateFlags       createFlags;
	//	XrApplicationInfo           applicationInfo;
	//	uint32_t                    enabledApiLayerCount;
	//	const char* const* enabledApiLayerNames;
	//	uint32_t                    enabledExtensionCount;
	//	const char* const* enabledExtensionNames;
	//} XrInstanceCreateInfo;

	XrInstanceCreateFlags       CreateFlags;
	XrApplicationInfo           ApplicationInfo;
	TArray<TStaticArray<ANSICHAR, XR_MAX_API_LAYER_NAME_SIZE>> EnabledLayerNames;
	TArray<TStaticArray<ANSICHAR, XR_MAX_EXTENSION_NAME_SIZE>> EnabledExtensionNames;

	XrInstance GeneratedInstance;

	//// TODO Support XrInstanceCreateInfoAndroidKHR on Android??
	// TODO: How do we handle next pointers?? Easiest solution is to always encode all possible next structs
	// into this packet.
	// alternatively, we could encode the packets after, and just have the decode routine _know_ to peek ahead?

	FOpenXRCreateInstancePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRCreateInstancePacket& Packet);
};

struct FOpenXRDestroyInstancePacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;

	FOpenXRDestroyInstancePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRDestroyInstancePacket& Packet);
};

struct FOpenXRGetInstancePropertiesPacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;
	XrInstanceProperties InstanceProperties;

	FOpenXRGetInstancePropertiesPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRGetInstancePropertiesPacket& Packet);
};

struct FOpenXRGetSystemPacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;
	XrSystemGetInfo SystemGetInfo;
	XrSystemId SystemId;

	FOpenXRGetSystemPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRGetSystemPacket& Packet);
};

struct FOpenXRGetSystemPropertiesPacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;
	XrSystemId SystemId;
	XrSystemProperties SystemProperties;

	FOpenXRGetSystemPropertiesPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRGetSystemPropertiesPacket& Packet);
};

struct FOpenXREnumerateEnvironmentBlendModesPacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;
	XrSystemId SystemId;
	XrViewConfigurationType ViewConfigurationType;
	TArray<XrEnvironmentBlendMode> EnvironmentBlendModes;

	FOpenXREnumerateEnvironmentBlendModesPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXREnumerateEnvironmentBlendModesPacket& Packet);
};

struct FOpenXRCreateSessionPacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;
	XrSessionCreateInfo SessionCreateInfo;
	XrSession Session;

	FOpenXRCreateSessionPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRCreateSessionPacket& Packet);
};

struct FOpenXRDestroySessionPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;

	FOpenXRDestroySessionPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRDestroySessionPacket& Packet);
};

struct FOpenXREnumerateReferenceSpacesPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	TArray<XrReferenceSpaceType> Spaces;

	FOpenXREnumerateReferenceSpacesPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXREnumerateReferenceSpacesPacket& Packet);
};

struct FOpenXRCreateReferenceSpacePacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrReferenceSpaceCreateInfo ReferenceSpaceCreateInfo;
	XrSpace Space;

	FOpenXRCreateReferenceSpacePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRCreateReferenceSpacePacket& Packet);
};

struct FOpenXRGetReferenceSpaceBoundsRectPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrReferenceSpaceType ReferenceSpaceType;
	XrExtent2Df Bounds;

	FOpenXRGetReferenceSpaceBoundsRectPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRGetReferenceSpaceBoundsRectPacket& Packet);
};

struct FOpenXRCreateActionSpacePacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrActionSpaceCreateInfo ActionSpaceCreateInfo;
	XrSpace Space;

	FOpenXRCreateActionSpacePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRCreateActionSpacePacket& Packet);
};

struct FOpenXRLocateSpacePacket : public FOpenXRAPIPacketBase
{
	XrSpace Space;
	XrSpace BaseSpace;
	XrTime Time;
	XrSpaceLocation Location;

	FOpenXRLocateSpacePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRLocateSpacePacket& Packet);
};

struct FOpenXRDestroySpacePacket : public FOpenXRAPIPacketBase
{
	XrSpace Space;

	FOpenXRDestroySpacePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRDestroySpacePacket& Packet);
};

struct FOpenXREnumerateViewConfigurationsPacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;
	XrSystemId SystemId;
	TArray<XrViewConfigurationType> ViewConfigurationTypes;

	FOpenXREnumerateViewConfigurationsPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXREnumerateViewConfigurationsPacket& Packet);
};

struct FOpenXRGetViewConfigurationPropertiesPacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;
	XrSystemId SystemId;
	XrViewConfigurationType ViewConfigurationType;
	XrViewConfigurationProperties ConfigurationProperties;

	FOpenXRGetViewConfigurationPropertiesPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRGetViewConfigurationPropertiesPacket& Packet);
};

struct FOpenXREnumerateViewConfigurationViewsPacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;
	XrSystemId SystemId;
	XrViewConfigurationType ViewConfigurationType;
	TArray<XrViewConfigurationView> Views;

	FOpenXREnumerateViewConfigurationViewsPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXREnumerateViewConfigurationViewsPacket& Packet);
};

struct FOpenXREnumerateSwapchainFormatsPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	TArray<int64> Formats;

	FOpenXREnumerateSwapchainFormatsPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXREnumerateSwapchainFormatsPacket& Packet);
};

struct FOpenXRCreateSwapchainPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrSwapchainCreateInfo SwapchainCreateInfo;
	XrSwapchain Swapchain;

	FOpenXRCreateSwapchainPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRCreateSwapchainPacket& Packet);
};

struct FOpenXRDestroySwapchainPacket : public FOpenXRAPIPacketBase
{
	XrSwapchain Swapchain;

	FOpenXRDestroySwapchainPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRDestroySwapchainPacket& Packet);
};

struct FOpenXREnumerateSwapchainImagesPacket : public FOpenXRAPIPacketBase
{
	XrSwapchain Swapchain;
	TArray<XrSwapchainImageBaseHeader> Images; // This will have...another 'real' array of images tied to API

	FOpenXREnumerateSwapchainImagesPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXREnumerateSwapchainImagesPacket& Packet);
};

struct FOpenXRAcquireSwapchainImagePacket : public FOpenXRAPIPacketBase
{
	XrSwapchain Swapchain;
	XrSwapchainImageAcquireInfo AcquireInfo;
	uint32 Index;

	FOpenXRAcquireSwapchainImagePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRAcquireSwapchainImagePacket& Packet);
};

struct FOpenXRWaitSwapchainImagePacket : public FOpenXRAPIPacketBase
{
	XrSwapchain Swapchain;
	XrSwapchainImageWaitInfo WaitInfo;

	FOpenXRWaitSwapchainImagePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRWaitSwapchainImagePacket& Packet);
};

struct FOpenXRReleaseSwapchainImagePacket : public FOpenXRAPIPacketBase
{
	XrSwapchain Swapchain;
	XrSwapchainImageReleaseInfo ReleaseInfo;

	FOpenXRReleaseSwapchainImagePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRReleaseSwapchainImagePacket& Packet);
};

struct FOpenXRBeginSessionPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrSessionBeginInfo SessionBeginInfo;

	FOpenXRBeginSessionPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRBeginSessionPacket& Packet);
};

struct FOpenXREndSessionPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;

	FOpenXREndSessionPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXREndSessionPacket& Packet);
};

struct FOpenXRRequestExitSessionPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;

	FOpenXRRequestExitSessionPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRRequestExitSessionPacket& Packet);
};

struct FOpenXRWaitFramePacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrFrameWaitInfo FrameWaitInfo;
	XrFrameState FrameState;

	FOpenXRWaitFramePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRWaitFramePacket& Packet);
};

struct FOpenXRBeginFramePacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrFrameBeginInfo FrameBeginInfo;

	FOpenXRBeginFramePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRBeginFramePacket& Packet);
};

struct FOpenXREndFramePacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	//typedef struct XrFrameEndInfo {
	//	XrStructureType                               type;
	//	const void* XR_MAY_ALIAS                      next;
	//	XrTime                                        displayTime;
	//	XrEnvironmentBlendMode                        environmentBlendMode;
	//	uint32_t                                      layerCount;
	//	const XrCompositionLayerBaseHeader* const* layers;
	//} XrFrameEndInfo;
	XrTime DisplayTime;
	XrEnvironmentBlendMode EnvironmentBlendMode;
	TArray<XrCompositionLayerBaseHeader> Layers; // source memory is not contiguous
	TArray<XrCompositionLayerQuad> QuadLayers;
	TArray<XrCompositionLayerProjection> ProjectionLayers;
	TArray<XrCompositionLayerProjectionView> ProjectionViews;

	FOpenXREndFramePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXREndFramePacket& Packet);
};

struct FOpenXRLocateViewsPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrViewLocateInfo ViewLocateInfo;
	XrViewState ViewState;
	TArray<XrView> Views;

	FOpenXRLocateViewsPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRLocateViewsPacket& Packet);
};

struct FOpenXRStringToPathPacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;
	TStaticArray<ANSICHAR, XR_MAX_PATH_LENGTH> PathStringToWrite;
	XrPath GeneratedPath;

	FOpenXRStringToPathPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRStringToPathPacket& Packet);
};

struct FOpenXRPathToStringPacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;
	XrPath ExistingPath;
	TStaticArray<ANSICHAR, XR_MAX_PATH_LENGTH> PathStringToRead;

	FOpenXRPathToStringPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRPathToStringPacket& Packet);
};

struct FOpenXRCreateActionSetPacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;
	XrActionSetCreateInfo ActionSetCreateInfo;
	XrActionSet ActionSet;

	FOpenXRCreateActionSetPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRCreateActionSetPacket& Packet);
};

struct FOpenXRDestroyActionSetPacket : public FOpenXRAPIPacketBase
{
	XrActionSet ActionSet;

	FOpenXRDestroyActionSetPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRDestroyActionSetPacket& Packet);
};

struct FOpenXRCreateActionPacket : public FOpenXRAPIPacketBase
{
	XrActionSet ActionSet;
	//typedef struct XrActionCreateInfo {
	//	XrStructureType             type;
	//	const void* XR_MAY_ALIAS    next;
	//	char                        actionName[XR_MAX_ACTION_NAME_SIZE];
	//	XrActionType                actionType;
	//	uint32_t                    countSubactionPaths;
	//	const XrPath* subactionPaths;
	//	char                        localizedActionName[XR_MAX_LOCALIZED_ACTION_NAME_SIZE];
	//} XrActionCreateInfo;
	TStaticArray<ANSICHAR, XR_MAX_ACTION_NAME_SIZE> ActionName;
	XrActionType ActionType;
	TArray<XrPath> SubactionPaths;
	TStaticArray<ANSICHAR, XR_MAX_LOCALIZED_ACTION_NAME_SIZE> LocalizedActionName;

	XrAction Action;

	FOpenXRCreateActionPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRCreateActionPacket& Packet);
};

struct FOpenXRDestroyActionPacket : public FOpenXRAPIPacketBase
{
	XrAction Action;

	FOpenXRDestroyActionPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRDestroyActionPacket& Packet);
};

struct FOpenXRSuggestInteractionProfileBindingsPacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;

	//typedef struct XrInteractionProfileSuggestedBinding {
	//	XrStructureType                    type;
	//	const void* XR_MAY_ALIAS           next;
	//	XrPath                             interactionProfile;
	//	uint32_t                           countSuggestedBindings;
	//	const XrActionSuggestedBinding* suggestedBindings;
	//} XrInteractionProfileSuggestedBinding;
	XrPath InteractionProfile;
	TArray<XrActionSuggestedBinding> SuggestedBindings;

	FOpenXRSuggestInteractionProfileBindingsPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRSuggestInteractionProfileBindingsPacket& Packet);
};

struct FOpenXRAttachSessionActionSetsPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	//typedef struct XrSessionActionSetsAttachInfo {
	//	XrStructureType             type;
	//	const void* XR_MAY_ALIAS    next;
	//	uint32_t                    countActionSets;
	//	const XrActionSet* actionSets;
	//} XrSessionActionSetsAttachInfo;
	TArray<XrActionSet> ActionSets;

	FOpenXRAttachSessionActionSetsPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRAttachSessionActionSetsPacket& Packet);
};

struct FOpenXRGetCurrentInteractionProfilePacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrPath TopLevelUserPath;
	XrInteractionProfileState InteractionProfile;

	FOpenXRGetCurrentInteractionProfilePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRGetCurrentInteractionProfilePacket& Packet);
};

struct FOpenXRGetActionStateBooleanPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrActionStateGetInfo GetInfoBoolean;
	XrActionStateBoolean BooleanState;

	FOpenXRGetActionStateBooleanPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRGetActionStateBooleanPacket& Packet);
};

struct FOpenXRGetActionStateFloatPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrActionStateGetInfo GetInfoFloat;
	XrActionStateFloat FloatState;

	FOpenXRGetActionStateFloatPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRGetActionStateFloatPacket& Packet);
};

struct FOpenXRGetActionStateVector2fPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrActionStateGetInfo GetInfoVector2f;
	XrActionStateVector2f Vector2fState;

	FOpenXRGetActionStateVector2fPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRGetActionStateVector2fPacket& Packet);
};

struct FOpenXRGetActionStatePosePacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrActionStateGetInfo GetInfoPose;
	XrActionStatePose PoseState;

	FOpenXRGetActionStatePosePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRGetActionStatePosePacket& Packet);
};

struct FOpenXRSyncActionsPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	//typedef struct XrActionsSyncInfo {
	//	XrStructureType             type;
	//	const void* XR_MAY_ALIAS    next;
	//	uint32_t                    countActiveActionSets;
	//	const XrActiveActionSet* activeActionSets;
	//} XrActionsSyncInfo;
	TArray<XrActiveActionSet> ActiveActionSets;

	FOpenXRSyncActionsPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRSyncActionsPacket& Packet);
};

struct FOpenXREnumerateBoundSourcesForActionPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrBoundSourcesForActionEnumerateInfo EnumerateInfo;
	TArray<XrPath> Sources;

	FOpenXREnumerateBoundSourcesForActionPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXREnumerateBoundSourcesForActionPacket& Packet);
};

struct FOpenXRGetInputSourceLocalizedNamePacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrInputSourceLocalizedNameGetInfo NameGetInfo;
	TStaticArray<ANSICHAR, XR_MAX_PATH_LENGTH> LocalizedName; // Not sure if this size is correct

	FOpenXRGetInputSourceLocalizedNamePacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRGetInputSourceLocalizedNamePacket& Packet);
};

struct FOpenXRApplyHapticFeedbackPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrHapticActionInfo HapticActionInfo;
	XrHapticBaseHeader HapticFeedback; // This is actually a different struct underneath

	FOpenXRApplyHapticFeedbackPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRApplyHapticFeedbackPacket& Packet);
};

struct FOpenXRStopHapticFeedbackPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrHapticActionInfo HapticActionInfo;

	FOpenXRStopHapticFeedbackPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRStopHapticFeedbackPacket& Packet);
};

// XR_KHR_loader_init
struct FOpenXRInitializeLoaderKHRPacket : public FOpenXRAPIPacketBase
{
	XrLoaderInitInfoBaseHeaderKHR LoaderInitInfo;

	// TODO: Encode the real data

	FOpenXRInitializeLoaderKHRPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRInitializeLoaderKHRPacket& Packet);
};

// XR_KHR_visibility_mask
struct FOpenXRGetVisibilityMaskKHRPacket : public FOpenXRAPIPacketBase
{
	XrSession Session;
	XrViewConfigurationType ViewConfigurationType;
	uint32_t ViewIndex;
	XrVisibilityMaskTypeKHR VisibilityMaskType;

	//typedef struct XrVisibilityMaskKHR {
	//	XrStructureType       type;
	//	void* XR_MAY_ALIAS    next;
	//	uint32_t              vertexCapacityInput;
	//	uint32_t              vertexCountOutput;
	//	XrVector2f* vertices;
	//	uint32_t              indexCapacityInput;
	//	uint32_t              indexCountOutput;
	//	uint32_t* indices;
	//} XrVisibilityMaskKHR;
	TArray<XrVector2f> Vertices;
	TArray<uint32> Indices;

	FOpenXRGetVisibilityMaskKHRPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRGetVisibilityMaskKHRPacket& Packet);
};

#if defined(XR_USE_GRAPHICS_API_D3D11)
// XR_KHR_D3D11_enable
struct FOpenXRGetD3D11GraphicsRequirementsKHRPacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;
	XrSystemId SystemId;
	XrGraphicsRequirementsD3D11KHR GraphicsRequirementsD3D11;

	FOpenXRGetD3D11GraphicsRequirementsKHRPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRGetD3D11GraphicsRequirementsKHRPacket& Packet);
};
#endif

#if defined(XR_USE_GRAPHICS_API_D3D12)
// XR_KHR_D3D12_enable
struct FOpenXRGetD3D12GraphicsRequirementsKHRPacket : public FOpenXRAPIPacketBase
{
	XrInstance Instance;
	XrSystemId SystemId;
	XrGraphicsRequirementsD3D12KHR GraphicsRequirementsD3D12;

	FOpenXRGetD3D12GraphicsRequirementsKHRPacket(XrResult InResult);

	friend FArchive& operator<<(FArchive& Ar, FOpenXRGetD3D12GraphicsRequirementsKHRPacket& Packet);
};
#endif
} // namespace UE::XRScribe
