// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;

namespace UnrealBuildTool.Rules
{
	// TODO Possibly split into capture module and replay module??
	public class XRScribe : ModuleRules
	{
		public XRScribe(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"OpenXR",
				"OpenXRHMD",
				"RHI"
			});

			PrivateDependencyModuleNames.AddRange(new string[]
			{
				"DeveloperSettings",
				"Projects",
			});

			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				PrivateDefinitions.Add("OPENXRHMD_HAVE_DX11");
				PrivateDefinitions.Add("OPENXRHMD_HAVE_DX12");
			}
		}
	}
}
